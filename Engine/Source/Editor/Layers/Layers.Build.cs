// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class Layers : ModuleRules
{
	public Layers(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"SceneOutliner",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"UnrealEd",
			}
		);
	}
}
