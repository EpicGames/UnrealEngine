// Copyright Epic Games, Inc. All Rights Reserved.
#include "ActorLayerCollectionViewModel.h"
#include "Editor/EditorEngine.h"
#include "Layers/Layer.h"
#include "ScopedTransaction.h"
#include "ActorLayerViewModel.h"


#define LOCTEXT_NAMESPACE "LayersView"

DEFINE_LOG_CATEGORY_STATIC(LogActorLayerCollectionViewModel, Fatal, All);

FActorLayerCollectionViewModel::FActorLayerCollectionViewModel( const TWeakObjectPtr< UEditorEngine >& InEditor )
	: bIsRefreshing( false )
	, Editor(InEditor)
	, WorldLayers(Editor.IsValid() ? Editor->GetEditorSubsystem<ULayersSubsystem>() : nullptr)
{
	// Sanity checks
	if (!Editor.IsValid())
	{
		UE_LOGF(LogActorLayerCollectionViewModel, Fatal, "This function requires Editor.IsValid() == true.");
	}
	else if (WorldLayers == nullptr)
	{
		UE_LOGF(LogActorLayerCollectionViewModel, Fatal, "This function requires Editor->GetEditorSubsystem<ULayersSubsystem>() to be already loaded rather than being a nullptr.");
	}
}


FActorLayerCollectionViewModel::~FActorLayerCollectionViewModel()
{
	if ( Editor.IsValid() )
	{
		WorldLayers->OnLayersChanged().RemoveAll(this);

		Editor->UnregisterForUndo( this );
	}
}


void FActorLayerCollectionViewModel::Initialize()
{
	if ( Editor.IsValid() )
	{
		WorldLayers->OnLayersChanged().AddSP(this, &FActorLayerCollectionViewModel::OnLayersChanged);

		Editor->RegisterForUndo( this );
	}

	RefreshLayers();
}


TArray< TSharedPtr< FActorLayerViewModel > >& FActorLayerCollectionViewModel::GetLayers()
{ 
	return Layers;
}


const TArray< TWeakObjectPtr< AActor > >& FActorLayerCollectionViewModel::GetActors() const 
{ 
	return Actors;
}


void FActorLayerCollectionViewModel::SetActors( const TArray< TWeakObjectPtr< AActor > >& InActors ) 
{ 
	Actors.Empty(); 
	Actors.Append( InActors ); 
	Refresh();
}


void FActorLayerCollectionViewModel::Refresh()
{
	OnLayersChanged( ELayersAction::Reset, NULL, NAME_None );
}


void FActorLayerCollectionViewModel::OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
{
	check( !bIsRefreshing );
	bIsRefreshing = true;

	switch ( Action )
	{
		case ELayersAction::Add:
			{
				if( ChangedLayer.IsValid() )
				{
					TSharedRef< FActorLayerViewModel > NewFLayer = FActorLayerViewModel::Create( ChangedLayer, Actors, Editor );

					if( DoAllActorsBelongtoLayer( NewFLayer ) )
					{
						Layers.Add( NewFLayer );
						SortLayers();
					}
				}
				else
				{
					RefreshLayers();
				}
			}
			break;

		case ELayersAction::Rename:
			{
				SortLayers();
			}
			break;

		case ELayersAction::Modify:
		case ELayersAction::Delete:
		case ELayersAction::Reset:
		default:
			{
				RefreshLayers();
			}
			break;
	}

	LayersChanged.Broadcast( Action, ChangedLayer, ChangedProperty );
	bIsRefreshing = false;
}


void FActorLayerCollectionViewModel::RefreshLayers()
{
	Layers.Empty();

	TArray< TWeakObjectPtr< ULayer > > TempLayers;
	WorldLayers->AddAllLayersTo( TempLayers );

	for( auto LayersIt = TempLayers.CreateIterator(); LayersIt; ++LayersIt )
	{
		TSharedRef< FActorLayerViewModel > NewFLayer = FActorLayerViewModel::Create( *LayersIt, Actors, Editor );

		if( DoAllActorsBelongtoLayer( NewFLayer ) )
		{
			Layers.Add( NewFLayer );
		}
	}

	SortLayers();
}


bool FActorLayerCollectionViewModel::DoAllActorsBelongtoLayer( const TSharedRef< FActorLayerViewModel >& Layer )
{
	if( Actors.Num() == 0 )
	{
		return false;
	}

	bool result = true;
	for( auto ActorIt = Actors.CreateConstIterator(); result && ActorIt; ++ActorIt )
	{
		const auto& Actor = *ActorIt;

		if( Actor.IsValid() )
		{
			result = Actor->Layers.Contains( Layer->GetFName() );
		}
	}

	return result;
}


void FActorLayerCollectionViewModel::SortLayers()
{
	struct FCompareLayers
	{
		FORCEINLINE bool operator()( const TSharedPtr< FActorLayerViewModel >& Lhs, const TSharedPtr< FActorLayerViewModel >& Rhs ) const 
		{ 
			return Lhs->GetFName().Compare( Rhs->GetFName() ) < 0;
		}
	};

	Layers.Sort( FCompareLayers() );
}


void FActorLayerCollectionViewModel::AppendLayerNames( TArray< FName >& OutLayerNames )
{
	for(auto LayersIt = Layers.CreateIterator(); LayersIt; ++LayersIt)
	{
		const TSharedPtr< FActorLayerViewModel >& Layer = *LayersIt;
		OutLayerNames.Add( Layer->GetFName() );
	}
}


void FActorLayerCollectionViewModel::RemoveActorsFromAllLayers()
{
	TArray< FName > LayerNames;
	AppendLayerNames( OUT LayerNames );

	const FScopedTransaction Transaction( LOCTEXT("RemoveActorsFromAllLayers", "Remove Actors from Layers") );
	WorldLayers->RemoveActorsFromLayers( Actors, LayerNames );
}


void FActorLayerCollectionViewModel::RemoveActorsFromLayer(  const TSharedPtr< FActorLayerViewModel >& Layer  )
{
	const FScopedTransaction Transaction( LOCTEXT("RemoveActorsFromLayer", "Remove Actors from Layer") );
	WorldLayers->RemoveActorsFromLayer( Actors, Layer->GetFName() );
}


#undef LOCTEXT_NAMESPACE
