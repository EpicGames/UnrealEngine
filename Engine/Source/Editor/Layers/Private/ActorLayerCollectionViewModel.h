// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "Layers/LayersSubsystem.h"

class AActor;
class FActorLayerViewModel;
class UEditorEngine;
class ULayer;

/**
 * The non-UI solution specific presentation logic for a collection of layers that belong to a collection of actors
 */
class FActorLayerCollectionViewModel : public TSharedFromThis< FActorLayerCollectionViewModel >, public FEditorUndoClient
{

public:
	
	/** FLayerCloud destructor */
	virtual ~FActorLayerCollectionViewModel();

	/**  
	 *	Factory method which creates a new FActorLayerCollectionViewModel object
	 *
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef< FActorLayerCollectionViewModel > Create( const TWeakObjectPtr< UEditorEngine >& InEditor )
	{
		const TSharedRef< FActorLayerCollectionViewModel > ViewModel( new FActorLayerCollectionViewModel( InEditor ) );
		ViewModel->Initialize();

		return ViewModel;
	}


public:

	/** @return	The list of FLayer objects to be displayed */
	TArray< TSharedPtr< FActorLayerViewModel > >& GetLayers();

	/** @return	The Actors whose layers should be displayed */
	const TArray< TWeakObjectPtr< AActor > >& GetActors() const;

	/** 
	 *	Sets the specified array of Actors whose layers should be displayed
	 *
	 *	@param	InActors	The Actors whose layers should be displayed
	 */
	void SetActors( const TArray< TWeakObjectPtr< AActor > >& InActors );

	/**	Removes the set list of Actors from all of their currently assigned Layers */
	void RemoveActorsFromAllLayers();

	/**	
	 * Removes the set list of Actors from the specified Layer
	 *
	 * @param	Layer	The Layer to remove from the actors
	 */
	void RemoveActorsFromLayer( const TSharedPtr< FActorLayerViewModel >& Layer );

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/** Broadcasts whenever the number of layers changes */
	DECLARE_DERIVED_EVENT( FActorLayerCollectionViewModel, ULayersSubsystem::FOnLayersChanged, FOnLayersChanged );
	FOnLayersChanged& OnLayersChanged() { return LayersChanged; }


private:

	/**  
	 *	 in the LayerCloud Constructor
	 *
	 *	@param	InEditor		The UEditorEngine to use
	 */
	FActorLayerCollectionViewModel( const TWeakObjectPtr< UEditorEngine >& InEditor );

	/** Initializes the FActorLayerCollectionViewModel for use */
	void Initialize();

	/**	Refreshes any cached information */
	void Refresh();

	/** Refreshes the Layers list */
	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty );

	/** Refreshes the Layers list */
	void RefreshLayers();

	/**	Sorts the Layers list */
	void SortLayers();

	/** Appends names of the currently exposed layers to the specified array */
	void AppendLayerNames( TArray< FName >& OutLayerNames );

	/**	
	 *  Returns whether the specified layer has all the specified Actors assigned to it
	 *
	 *  @param	Layer		The layer to check actor assignment on
	 *	@return				Whether the Layer includes all the specified Actors
	 */
	bool DoAllActorsBelongtoLayer( const TSharedRef< FActorLayerViewModel >& Layer );

private:

	/** true if the in the middle of refreshing */
	bool bIsRefreshing;

	/** All layers shown in the LayersView */
	TArray< TSharedPtr< FActorLayerViewModel > > Layers;

	/**	All actors whose layers are being exposed */
	TArray< TWeakObjectPtr< AActor > > Actors;

	/** The UEditorEngine to use */
	const TWeakObjectPtr< UEditorEngine > Editor;

	/** The layer management logic object */
	ULayersSubsystem* WorldLayers;

	/**	Broadcasts whenever one or more layers changes */
	FOnLayersChanged LayersChanged;
};
