// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLayerViewModel.h"
#include "Editor/EditorEngine.h"
#include "Layers/Layer.h"

#define LOCTEXT_NAMESPACE "Layer"

FActorLayerViewModel::FActorLayerViewModel( const TWeakObjectPtr< ULayer >& InLayer, const TArray< TWeakObjectPtr< AActor > >& InActors, const TWeakObjectPtr< UEditorEngine >& InEditor )
	: Editor( InEditor )
	, Layer( InLayer )
{
	Actors.Append( InActors );
}


void FActorLayerViewModel::Initialize()
{
	if ( Editor.IsValid() )
	{
		ULayersSubsystem* WorldLayers = Editor->GetEditorSubsystem<ULayersSubsystem>();
		WorldLayers->OnLayersChanged().AddSP(this, &FActorLayerViewModel::OnLayersChanged);

		Editor->RegisterForUndo(this);
	}
}


FActorLayerViewModel::~FActorLayerViewModel()
{
	if ( Editor.IsValid() )
	{
		ULayersSubsystem* WorldLayers = Editor->GetEditorSubsystem<ULayersSubsystem>();
		WorldLayers->OnLayersChanged().RemoveAll(this);

		Editor->UnregisterForUndo(this);
	}
}


FName FActorLayerViewModel::GetFName() const
{
	if( !Layer.IsValid() )
	{
		return NAME_None;
	}

	return Layer->GetLayerName();
}


FText FActorLayerViewModel::GetName() const
{
	if( !Layer.IsValid() )
	{
		return FText::GetEmpty();
	}

	return FText::FromName(Layer->GetLayerName());
}


bool FActorLayerViewModel::IsVisible() const
{
	if( !Layer.IsValid() )
	{
		return false;
	}

	return Layer->IsVisible();
}


void FActorLayerViewModel::OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
{
	if( Action != ELayersAction::Modify && Action != ELayersAction::Reset )
	{
		return;
	}

	if( ChangedLayer.IsValid() && ChangedLayer != Layer )
	{
		return;
	}

	Changed.Broadcast();
}


void FActorLayerViewModel::Refresh()
{
	OnLayersChanged( ELayersAction::Reset, NULL, NAME_None );
}


#undef LOCTEXT_NAMESPACE
