// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "Layers/LayersSubsystem.h"

class AActor;
class UEditorEngine;
class ULayer;

/**
 * The non-UI solution specific presentation logic for a single Layer
 */
class FActorLayerViewModel : public TSharedFromThis< FActorLayerViewModel >, public FEditorUndoClient
{

public:
	
	/** FActorLayerViewModel destructor */
	virtual ~FActorLayerViewModel();

	/**  
	 *	Factory method which creates a new FActorLayerViewModel object
	 *
	 *	@param	InLayer			The layer wrap
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef< FActorLayerViewModel > Create( const TWeakObjectPtr< ULayer >& InLayer, const TArray< TWeakObjectPtr< AActor > >& InActors, const TWeakObjectPtr< UEditorEngine >& InEditor )
	{
		const TSharedRef< FActorLayerViewModel > NewLayer( new FActorLayerViewModel( InLayer, InActors, InEditor ) );
		NewLayer->Initialize();

		return NewLayer;
	}


public:
	
	/**	@return	The Layer's display name as a FName */
	FName GetFName() const;

	/**	@return	The Layer's display name as a FString */
	FText GetName() const;

	/**	@return Whether the Layer is visible */
	bool IsVisible() const;

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/** Broadcasts whenever the layer changes */
	FSimpleMulticastDelegate Changed;


private:

	/**
	 *	FActorLayerViewModel Constructor
	 *
	 *	@param	InLayer			The Layer to represent
	 *	@param	InEditor		The UEditorEngine to use
	 */
	FActorLayerViewModel( const TWeakObjectPtr< ULayer >& InLayer, const TArray< TWeakObjectPtr< AActor > >& InActors, const TWeakObjectPtr< UEditorEngine >& InEditor );

	/**	Initializes the FActorLayerViewModel for use */
	void Initialize();

	/**	Called when a change occurs regarding Layers */
	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty );

	/** Refreshes any cached information in the FActorLayerViewModel */
	void Refresh();


private:

	/** The UEditorEngine to use */
	const TWeakObjectPtr< UEditorEngine > Editor;

	/**	The Layer this object represents */
	const TWeakObjectPtr< ULayer > Layer;

	/**	The Actors this object represents */
	TArray< TWeakObjectPtr< AActor > > Actors;
};
