// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerFilters.h"
#include "ActorTreeItem.h"
#include "GameFramework/Actor.h"

class FActorsAssignedToSpecificLayersFilter : public TSceneOutlinerFilter<FActorTreeItem>, public TSharedFromThis< FActorsAssignedToSpecificLayersFilter >
{
public:

	/** By default, any types not handled by this filter will fail */
	FActorsAssignedToSpecificLayersFilter()
		: TSceneOutlinerFilter<FActorTreeItem>(FSceneOutlinerFilter::EDefaultBehaviour::Fail)
	{}

	/** 
	 * Returns whether the specified Item passes the Filter's text restrictions 
	 *
	 *	@param	InItem	The Item to check 
	 *	@return			Whether the specified Item passed the filter
	 */
	virtual bool PassesFilterImpl(const FActorTreeItem& InItem) const
	{
		const AActor* InActor = InItem.Actor.Get();
		if( LayerNames.Num() == 0 )
		{
			return false;
		}

		if( InActor->Layers.Num() < LayerNames.Num() )
		{
			return false;
		}

		bool bBelongsToLayers = true;
		for( auto LayerIter = LayerNames.CreateConstIterator(); bBelongsToLayers && LayerIter; ++LayerIter )
		{
			bBelongsToLayers = InActor->Layers.Contains( *LayerIter );
		}

		return bBelongsToLayers;
	}

	/**
	 *	
	 */
	void SetLayers( const TArray< FName >& InLayerNames )
	{
		LayerNames.Empty();

		for( auto LayerNameIter = InLayerNames.CreateConstIterator(); LayerNameIter; ++LayerNameIter )
		{
			LayerNames.AddUnique( *LayerNameIter );
		}

		ChangedEvent.Broadcast();
	}

	/**
	 *	
	 */
	void AddLayer( FName LayerName )
	{
		LayerNames.AddUnique( LayerName );
		ChangedEvent.Broadcast();
	}

	/**
	 *	
	 */
	bool RemoveLayer( FName LayerName )
	{
		return LayerNames.Remove( LayerName ) > 0;
	}

	/**
	 *	
	 */
	void ClearLayers()
	{
		LayerNames.Empty();
		ChangedEvent.Broadcast();
	}


private:

	/**	The list of layer names which actors need to belong to */
	TArray< FName > LayerNames;
};
