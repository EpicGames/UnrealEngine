// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "LayersViewCommands"

/**
 * The set of commands supported by the LayersView
 */
class FLayersViewCommands : public TCommands<FLayersViewCommands>
{

public:

	/** FLayersViewCommands Constructor */
	FLayersViewCommands() : TCommands<FLayersViewCommands>
	(
		"LayersView", // Context name for fast lookup
		NSLOCTEXT("Contexts", "LayersView", "Layers"), // Localized context name for displaying
		"LevelEditor", // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{
	}
	
	/**
	 * Initialize the commands
	 */
	virtual void RegisterCommands() override
	{
		UI_COMMAND( CreateEmptyLayer, "Create Empty Layer", "Creates a new empty Layer", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddSelectedActorsToNewLayer, "Add Selected Actors to New Layer", "Adds the actors currently selected in the active viewport to a new layer", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddSelectedActorsToSelectedLayer, "Add Selected Actors to Selected Layers", "Adds the actors currently selected in the viewport to the selected layers", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( RemoveSelectedActorsFromSelectedLayer, "Remove Selected Actors from Layers", "Removes the actors currently selected in the viewport from the selected layers", EUserInterfaceActionType::Button, FInputChord() );

		UI_COMMAND( SelectActors, "Select Actors", "Actors in the selected Layers becomes the viewport's selection", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AppendActorsToSelection, "Append Actors to Selection", "Adds the Actors in the selected Layers to the viewport's existing selection", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( DeselectActors, "Deselect Actors", "Removes the Actors in the selected Layers from the viewport's existing selection", EUserInterfaceActionType::Button, FInputChord() );

		UI_COMMAND( ToggleSelectedLayersVisibility, "Toggle Visibility of Selected Layers", "Toggles the visibility of the selected layers in the viewports", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( MakeAllLayersVisible, "Make All Layers Visible", "Toggles all layers to a visible state in the viewports", EUserInterfaceActionType::Button, FInputChord() );

		UI_COMMAND( RequestRenameLayer, "Rename", "Rename the selected layer.", EUserInterfaceActionType::Button, FInputChord( EKeys::F2 ) );	
	}

public:

	/** Creates a new empty Layer */
	TSharedPtr< FUICommandInfo > CreateEmptyLayer;

	/** Adds the Selected Actors To a New Layer */
	TSharedPtr< FUICommandInfo > AddSelectedActorsToNewLayer;

	/** Adds the Selected Actors to the Selected Layers */
	TSharedPtr< FUICommandInfo > AddSelectedActorsToSelectedLayer;

	/** Removes the Selected Actors from the Selected Layers */
	TSharedPtr< FUICommandInfo > RemoveSelectedActorsFromSelectedLayer;

	/** Selects the actors in the Selected Layers */
	TSharedPtr< FUICommandInfo > SelectActors;

	/** Adds the actors in the Selected Layers to the currently selected Actors */
	TSharedPtr< FUICommandInfo > AppendActorsToSelection;

	/** Deselects the actors in the Selected Layers */
	TSharedPtr< FUICommandInfo > DeselectActors;

	/** Toggles the visibility of the currently selected layers */
	TSharedPtr< FUICommandInfo > ToggleSelectedLayersVisibility;

	/** Makes all layers visible */
	TSharedPtr< FUICommandInfo > MakeAllLayersVisible;

	/** Requests a rename on the layer */
	TSharedPtr< FUICommandInfo > RequestRenameLayer;
};

#undef LOCTEXT_NAMESPACE
