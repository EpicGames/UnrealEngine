// Copyright Epic Games, Inc. All Rights Reserved.
#include "LayerCollectionViewModel.h"
#include "Editor/EditorEngine.h"
#include "Misc/FilterCollection.h"
#include "ScopedTransaction.h"
#include "LayerViewModel.h"
#include "LayerCollectionViewCommands.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "LayersView"

DEFINE_LOG_CATEGORY_STATIC(LogLayerCollectionViewModel, Fatal, All);

FLayerCollectionViewModel::FLayerCollectionViewModel( const TWeakObjectPtr< UEditorEngine >& InEditor )
	: bIsRefreshing( false )
	, Filters( MakeShareable( new LayerFilterCollection ) )
	, CommandList( MakeShareable( new FUICommandList ) )
	, Editor( InEditor )
	, WorldLayers(Editor.IsValid() ? Editor->GetEditorSubsystem<ULayersSubsystem>() : nullptr)
{
	// Sanity checks
	if (!Editor.IsValid())
	{
		UE_LOGF(LogLayerCollectionViewModel, Fatal, "This function requires Editor.IsValid() == true.");
	}
	else if (WorldLayers == nullptr)
	{
		UE_LOGF(LogLayerCollectionViewModel, Fatal, "This function requires Editor->GetEditorSubsystem<ULayersSubsystem>() to be already loaded rather than being a nullptr.");
	}
}


FLayerCollectionViewModel::~FLayerCollectionViewModel()
{
	Filters->OnChanged().RemoveAll( this );
	WorldLayers->OnLayersChanged().RemoveAll( this );

	if ( Editor.IsValid() )
	{
		Editor->UnregisterForUndo( this );
	}
}


void FLayerCollectionViewModel::Initialize()
{
	BindCommands();

	Filters->OnChanged().AddSP( this, &FLayerCollectionViewModel::OnFilterChanged );
	WorldLayers->OnLayersChanged().AddSP( this, &FLayerCollectionViewModel::OnLayersChanged );

	if ( Editor.IsValid() )
	{
		Editor->RegisterForUndo( this );
	}

	Refresh(); 
}


void FLayerCollectionViewModel::BindCommands()
{
	const FLayersViewCommands& Commands = FLayersViewCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction( FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::DeleteLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::DeleteLayer_CanExecute ) );

	ActionList.MapAction( Commands.AddSelectedActorsToSelectedLayer,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AddSelectedActorsToSelectedLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AddSelectedActorsToSelectedLayer_CanExecute ) );

	ActionList.MapAction( Commands.CreateEmptyLayer,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::CreateEmptyLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::CreateEmptyLayer_CanExecute ) );

	ActionList.MapAction( Commands.AddSelectedActorsToNewLayer,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AddSelectedActorsToNewLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AddSelectedActorsToNewLayer_CanExecute ) );

	ActionList.MapAction( Commands.RemoveSelectedActorsFromSelectedLayer,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::RemoveSelectedActorsFromSelectedLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::RemoveSelectedActorsFromSelectedLayer_CanExecute ) );


	ActionList.MapAction( Commands.SelectActors,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::SelectActors_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::SelectActors_CanExecute ) );

	ActionList.MapAction( Commands.AppendActorsToSelection,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AppendActorsToSelection_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::AppendActorsToSelection_CanExecute ) );

	ActionList.MapAction( Commands.DeselectActors,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::DeselectActors_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::DeselectActors_CanExecute ) );


	ActionList.MapAction( Commands.ToggleSelectedLayersVisibility,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::ToggleSelectedLayersVisibility_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::ToggleSelectedLayersVisibility_CanExecute ) );

	ActionList.MapAction( Commands.MakeAllLayersVisible,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::MakeAllLayersVisible_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::MakeAllLayersVisible_CanExecute ) );


	ActionList.MapAction( Commands.RequestRenameLayer,
		FExecuteAction::CreateSP( this, &FLayerCollectionViewModel::RequestRenameLayer_Executed ),
		FCanExecuteAction::CreateSP( this, &FLayerCollectionViewModel::RequestRenameLayer_CanExecute ) );
}


void FLayerCollectionViewModel::AddFilter( const TSharedRef< LayerFilter >& InFilter )
{
	Filters->Add( InFilter );
	OnFilterChanged();
}


void FLayerCollectionViewModel::RemoveFilter( const TSharedRef< LayerFilter >& InFilter )
{
	Filters->Remove( InFilter );
	OnFilterChanged();
}


TArray< TSharedPtr< FLayerViewModel > >& FLayerCollectionViewModel::GetLayers()
{ 
	return FilteredLayerViewModels;
}


const TArray< TSharedPtr< FLayerViewModel > >& FLayerCollectionViewModel::GetSelectedLayers() const 
{ 
	return SelectedLayers;
}


void FLayerCollectionViewModel::GetSelectedLayerNames( OUT TArray< FName >& OutSelectedLayerNames ) const
{
	AppendSelectLayerNames( OutSelectedLayerNames );
}


void FLayerCollectionViewModel::SetSelectedLayers( const TArray< TSharedPtr< FLayerViewModel > >& InSelectedLayers ) 
{ 
	SelectedLayers.Empty(); 
	SelectedLayers.Append( InSelectedLayers ); 
	SelectionChanged.Broadcast();
}


void FLayerCollectionViewModel::SetSelectedLayers( const TArray< FName >& LayerNames ) 
{ 
	SelectedLayers.Empty();
	for( auto LayerIter = FilteredLayerViewModels.CreateConstIterator(); LayerIter; ++LayerIter )
	{
		const auto LayerViewModel = *LayerIter;

		if( LayerNames.Contains( LayerViewModel->GetFName() ) )
		{
			SelectedLayers.Add( LayerViewModel );
		}
	}

	SelectionChanged.Broadcast();
}


void FLayerCollectionViewModel::SetSelectedLayer( const FName& LayerName ) 
{ 
	SelectedLayers.Empty();
	for( auto LayerIter = FilteredLayerViewModels.CreateConstIterator(); LayerIter; ++LayerIter )
	{
		const auto LayerViewModel = *LayerIter;

		if( LayerName == LayerViewModel->GetFName() )
		{
			SelectedLayers.Add( LayerViewModel );
			break;
		}
	}

	SelectionChanged.Broadcast();
}


const TSharedRef< FUICommandList > FLayerCollectionViewModel::GetCommandList() const 
{ 
	return CommandList;
}


void FLayerCollectionViewModel::OnFilterChanged()
{
	RefreshFilteredLayers();
	LayersChanged.Broadcast( ELayersAction::Reset, NULL, NAME_None );
}


void FLayerCollectionViewModel::Refresh()
{
	WorldLayers->UpdateAllActorsVisibility( true, true );
	OnLayersChanged( ELayersAction::Reset, NULL, NAME_None );
}


void FLayerCollectionViewModel::OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
{
	check( !bIsRefreshing );
	bIsRefreshing = true;

	switch ( Action )
	{
		case ELayersAction::Add:
			OnLayerAdded( ChangedLayer );
			break;

		case ELayersAction::Rename:
			//We purposely ignore re-filtering in this case
			SortFilteredLayers();
			break;

		case ELayersAction::Modify:
			RefreshFilteredLayers();
			break;

		case ELayersAction::Delete:
			OnLayerDelete();
			break;

		case ELayersAction::Reset:
		default:
			OnResetLayers();
			break;
	}

	LayersChanged.Broadcast( Action, ChangedLayer, ChangedProperty );
	bIsRefreshing = false;
}


void FLayerCollectionViewModel::OnResetLayers()
{
	TArray< TWeakObjectPtr< ULayer > > ActualLayers;
	WorldLayers->AddAllLayersTo( ActualLayers );

	FilteredLayerViewModels.Empty();

	//Purge any invalid viewmodels, 
	//this function also removes any layers already with viewmodel representations from ActualLayers
	DestructivelyPurgeInvalidViewModels( ActualLayers );

	//Create any missing viewmodels
	CreateViewModels( ActualLayers );

	//Rebuild the filtered layers list
	RefreshFilteredLayers();
}


void FLayerCollectionViewModel::OnLayerAdded( const TWeakObjectPtr< ULayer >& AddedLayer )
{
	if( !AddedLayer.IsValid() )
	{
		OnResetLayers();
		return;
	}

	const TSharedRef< FLayerViewModel > NewLayerViewModel = FLayerViewModel::Create( AddedLayer, Editor );
	NewLayerViewModel->OnVisibilityToggled().AddSP( this, &FLayerCollectionViewModel::ToggleLayerVisibility );
	AllLayerViewModels.Add( NewLayerViewModel );

	// We specifically ignore filters when dealing with single additions
	FilteredLayerViewModels.Add( NewLayerViewModel );
	SortFilteredLayers();
}


void FLayerCollectionViewModel::OnLayerDelete()
{
	TArray< TWeakObjectPtr< ULayer > > ActualLayers;
	WorldLayers->AddAllLayersTo( ActualLayers );

	DestructivelyPurgeInvalidViewModels( ActualLayers );
}


void FLayerCollectionViewModel::DestructivelyPurgeInvalidViewModels( TArray< TWeakObjectPtr< ULayer > >& InLayers ) 
{
	for( int LayerIndex = AllLayerViewModels.Num() - 1; LayerIndex >= 0; --LayerIndex )
	{
		const auto LayerViewModel = AllLayerViewModels[ LayerIndex ];
		const auto Layer = LayerViewModel->GetDataSource(); 

		//Remove any viewmodels with invalid datasources or whose datasources 
		//are no longer in the list of layers
		if( !Layer.IsValid() || InLayers.Remove( Layer ) == 0 )
		{
			AllLayerViewModels.RemoveAt( LayerIndex );
			FilteredLayerViewModels.Remove( LayerViewModel );
			SelectedLayers.Remove( LayerViewModel );
		}
	}
}


void FLayerCollectionViewModel::CreateViewModels( const TArray< TWeakObjectPtr< ULayer > >& InLayers ) 
{
	for( auto LayerIt = InLayers.CreateConstIterator(); LayerIt; ++LayerIt )
	{
		const TSharedRef< FLayerViewModel > NewLayerViewModel = FLayerViewModel::Create( *LayerIt, Editor );
		NewLayerViewModel->OnVisibilityToggled().AddSP( this, &FLayerCollectionViewModel::ToggleLayerVisibility );
		AllLayerViewModels.Add( NewLayerViewModel );

		if( Filters->PassesAllFilters( NewLayerViewModel ) )
		{
			FilteredLayerViewModels.Add( NewLayerViewModel );
		}
	}
}


void FLayerCollectionViewModel::RefreshFilteredLayers()
{
	FilteredLayerViewModels.Empty();

	for( auto LayerIt = AllLayerViewModels.CreateIterator(); LayerIt; ++LayerIt )
	{
		const auto LayerViewModel = *LayerIt;
		if( Filters->PassesAllFilters( LayerViewModel ) )
		{
			FilteredLayerViewModels.Add( LayerViewModel );
		}
	}

	SortFilteredLayers();
}


void FLayerCollectionViewModel::SortFilteredLayers()
{
	struct FCompareLayers
	{
		FORCEINLINE bool operator()( const TSharedPtr< FLayerViewModel >& Lhs, const TSharedPtr< FLayerViewModel >& Rhs ) const 
		{ 
			return Lhs->GetFName().Compare( Rhs->GetFName() ) < 0;
		}
	};

	FilteredLayerViewModels.Sort( FCompareLayers() );
}


void FLayerCollectionViewModel::AppendSelectLayerNames( TArray< FName >& OutLayerNames ) const
{
	for(auto LayersIt = SelectedLayers.CreateConstIterator(); LayersIt; ++LayersIt)
	{
		const TSharedPtr< FLayerViewModel >& Layer = *LayersIt;
		OutLayerNames.Add( Layer->GetFName() );
	}
}

void FLayerCollectionViewModel::AddActorsToNewLayer( TArray< TWeakObjectPtr< AActor > > Actors )
{
	const FScopedTransaction Transaction( LOCTEXT("AddActorsToNewLayer", "Add Selected Actors to New Layer") );
	const FName NewLayerName = GenerateUniqueLayerName();
	WorldLayers->AddActorsToLayer( Actors, NewLayerName );

	SetSelectedLayer( NewLayerName );
}

FName FLayerCollectionViewModel::GenerateUniqueLayerName() const
{
	FName DefaultName;
	int32 LayerIndex = 0;
	do
	{
		++LayerIndex;
		DefaultName = FName( *FString::Printf( TEXT("Layer%d"), LayerIndex ) );
	} 
	while ( WorldLayers->IsLayer( DefaultName ) );

	return DefaultName;
}


// DeleteLayer ----------------------------------------------------------------
void FLayerCollectionViewModel::DeleteLayer_Executed()
{
	if (SelectedLayers.Num() == 0)
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames(SelectedLayerNames);

	const FScopedTransaction Transaction(LOCTEXT("DeleteLayer", "Delete Layer"));

	WorldLayers->DeleteLayers(SelectedLayerNames);
}

bool FLayerCollectionViewModel::DeleteLayer_CanExecute() const
{
	return SelectedLayers.Num() > 0;
}


// CreateEmptyLayer -----------------------------------------------------------
void FLayerCollectionViewModel::CreateEmptyLayer_Executed()
{
	const FScopedTransaction Transaction( LOCTEXT("CreateEmptyLayer", "Create Empty Layer") );
	const FName NewLayerName = GenerateUniqueLayerName();
	WorldLayers->CreateLayer( NewLayerName );

	SetSelectedLayer( NewLayerName );

	if(RequestRenameLayer_CanExecute())
	{
		RequestRenameLayer_Executed();
	}
}

bool FLayerCollectionViewModel::CreateEmptyLayer_CanExecute() const
{
	return true;
}


// AddSelectedActorsToNewLayer ------------------------------------------------
void FLayerCollectionViewModel::AddSelectedActorsToNewLayer_Executed()
{
	const FScopedTransaction Transaction( LOCTEXT("AddSelectedActorsToNewLayer", "Add Actors to New Layer") );
	const FName NewLayerName = GenerateUniqueLayerName();
	WorldLayers->AddSelectedActorsToLayer( NewLayerName );

	SetSelectedLayer( NewLayerName );

	if(RequestRenameLayer_CanExecute())
	{
		RequestRenameLayer_Executed();
	}
}

bool FLayerCollectionViewModel::AddSelectedActorsToNewLayer_CanExecute() const
{
	return Editor->GetSelectedActorCount() > 0;
}


// AddSelectedActorsToSelectedLayer -------------------------------------------
void FLayerCollectionViewModel::AddSelectedActorsToSelectedLayer_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const FScopedTransaction Transaction( LOCTEXT("AddSelectedActorsToSelectedLayer", "Add Selected Actors to Layer") );

	WorldLayers->AddSelectedActorsToLayers( SelectedLayerNames );
}

bool FLayerCollectionViewModel::AddSelectedActorsToSelectedLayer_CanExecute() const
{
	return SelectedLayers.Num() > 0 && Editor->GetSelectedActorCount() > 0;
}


// RemoveSelectedActorsFromSelectedLayer --------------------------------------
void FLayerCollectionViewModel::RemoveSelectedActorsFromSelectedLayer_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const FScopedTransaction Transaction( LOCTEXT("RemoveSelectedActorsFromSelectedLayer", "Remove Selected Actors to Layer") );

	WorldLayers->RemoveSelectedActorsFromLayers( SelectedLayerNames );
}

bool FLayerCollectionViewModel::RemoveSelectedActorsFromSelectedLayer_CanExecute() const
{
	return SelectedLayers.Num() > 0 && Editor->GetSelectedActorCount() > 0;
}


// SelectActors ---------------------------------------------------------------
void FLayerCollectionViewModel::SelectActors_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("SelectActors", "Select Actors in Layer") );

	const bool bNotifySelectNone = false;
	const bool bDeselectBSPSurfs = true;
	Editor->SelectNone( bNotifySelectNone, bDeselectBSPSurfs );

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const bool bSelectActors = true;
	const bool bNotifySelectActors = true;
	const bool bSelectEvenIfHidden = true;
	WorldLayers->SelectActorsInLayers( SelectedLayerNames, bSelectActors, bNotifySelectActors, bSelectEvenIfHidden );
}

bool FLayerCollectionViewModel::SelectActors_CanExecute() const
{
	return SelectedLayers.Num() > 0;
}


// AppendActorsToSelection ----------------------------------------------------
void FLayerCollectionViewModel::AppendActorsToSelection_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const FScopedTransaction Transaction( LOCTEXT("AppendActorsToSelection", "Append Actors in Layer to Selection") );

	const bool bSelect = true;
	const bool bNotifySelectActors = true;
	const bool bSelectEvenIfHidden = true;
	WorldLayers->SelectActorsInLayers( SelectedLayerNames, bSelect, bNotifySelectActors, bSelectEvenIfHidden );
}

bool FLayerCollectionViewModel::AppendActorsToSelection_CanExecute() const
{
	return SelectedLayers.Num() > 0;
}


// DeselectActors -------------------------------------------------------------
void FLayerCollectionViewModel::DeselectActors_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const FScopedTransaction Transaction( LOCTEXT("DeselectActors", "Deselect Actors in Layer") );

	const bool bSelect = false;
	const bool bNotifySelectActors = true;
	WorldLayers->SelectActorsInLayers( SelectedLayerNames, bSelect, bNotifySelectActors );
}

bool FLayerCollectionViewModel::DeselectActors_CanExecute() const
{
	return SelectedLayers.Num() > 0;
}


// ToggleLayerVisibility ------------------------------------------------------
void FLayerCollectionViewModel::ToggleLayerVisibility( const TSharedPtr<FLayerViewModel>& InLayer )
{
	if ( SelectedLayers.Find( InLayer ) == INDEX_NONE )
	{
		// Given layer wasn't selected so toggle only its own visibility
		const FScopedTransaction Transaction( LOCTEXT( "ToggleVisibility", "Toggle Layer Visibility" ) );
		WorldLayers->ToggleLayerVisibility( InLayer->GetFName() );
	}
	else
	{
		// Toggle visibility of selected layers to the same visibility state as the given layer
		bool bVisible = InLayer->IsVisible();

		TArray< FName > SelectedLayerNames;
		for( auto LayersIt = SelectedLayers.CreateConstIterator(); LayersIt; ++LayersIt )
		{
			const TSharedPtr< FLayerViewModel >& Layer = *LayersIt;
			if ( Layer->IsVisible() == bVisible )
			{
				SelectedLayerNames.Add( Layer->GetFName() );
			}
		}

		const FScopedTransaction Transaction( LOCTEXT("ToggleSelectedLayersVisibility", "Toggle Layer Visibility") );
		WorldLayers->ToggleLayersVisibility( SelectedLayerNames );
	}
}


// ToggleSelectedLayersVisibility ---------------------------------------------
void FLayerCollectionViewModel::ToggleSelectedLayersVisibility_Executed()
{
	if( SelectedLayers.Num() == 0 )
	{
		return;
	}

	TArray< FName > SelectedLayerNames;
	AppendSelectLayerNames( SelectedLayerNames );

	const FScopedTransaction Transaction( LOCTEXT("ToggleSelectedLayersVisibility", "Toggle Layer Visibility") );

	WorldLayers->ToggleLayersVisibility( SelectedLayerNames );
}

bool FLayerCollectionViewModel::ToggleSelectedLayersVisibility_CanExecute() const
{
	return SelectedLayers.Num() > 0;
}


// MakeAllLayersVisible -------------------------------------------------------
void FLayerCollectionViewModel::MakeAllLayersVisible_Executed()
{
	const FScopedTransaction Transaction( LOCTEXT("MakeAllLayersVisible", "Make All Layers Visible") );
	WorldLayers->MakeAllLayersVisible();
}

bool FLayerCollectionViewModel::MakeAllLayersVisible_CanExecute() const
{
	return AllLayerViewModels.Num() > 0;
}


// RequestRenameLayer ---------------------------------------------------------
void FLayerCollectionViewModel::RequestRenameLayer_Executed()
{
	if(SelectedLayers.Num() == 1)
	{
		OnRenameRequested().Broadcast();
	}
}

bool FLayerCollectionViewModel::RequestRenameLayer_CanExecute() const
{
	return SelectedLayers.Num() == 1;
}

#undef LOCTEXT_NAMESPACE
