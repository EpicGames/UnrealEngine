// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorUndoClient.h"
#include "Layers/LayersSubsystem.h"

class AActor;
class FLayerViewModel;
class UEditorEngine;
class ULayer;
template< typename ItemType > class TFilterCollection;
template< typename TItemType > class IFilter;

typedef TFilterCollection< const TSharedPtr< FLayerViewModel >& > LayerFilterCollection;
typedef IFilter< const TSharedPtr< FLayerViewModel >& > LayerFilter;

/**
 * The non-UI solution specific presentation logic for a LayersView
 */
class FLayerCollectionViewModel : public TSharedFromThis< FLayerCollectionViewModel >, public FEditorUndoClient
{

public:
	
	/** FLayerCollectionViewModel destructor */
	virtual ~FLayerCollectionViewModel();

	/**  
	 *	Factory method which creates a new FLayerCollectionViewModel object
	 *
	 *	@param	InWorldLayers	The layer management logic object
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef< FLayerCollectionViewModel > Create( const TWeakObjectPtr< UEditorEngine >& InEditor )
	{
		TSharedRef< FLayerCollectionViewModel > LayersView( new FLayerCollectionViewModel( InEditor ) );
		LayersView->Initialize();

		return LayersView;
	}


public:

	/** 
	 *	Adds a filter which restricts the layers shown in the LayersView
	 *	
	 *	@param	InFilter	The Filter to add
	 */
	void AddFilter( const TSharedRef< LayerFilter >& InFilter );

	/** 
	 *	Removes a filter which restricted the layers shown in the LayersView
	 *
	 *	@param	InFilter	The Filter to remove		
	 */
	void RemoveFilter( const TSharedRef< LayerFilter >& InFilter );

	/** @return	The list of ULayer objects to be displayed in the LayersView */
	TArray< TSharedPtr< FLayerViewModel > >& GetLayers();

	/** @return	The selected ULayer objects in the LayersView */
	const TArray< TSharedPtr< FLayerViewModel > >& GetSelectedLayers() const;

	/**
	 *	Appends the names of the currently selected layers to the provided array
	 *
	 * @param	OutSelectedLayerNames	The array to append the layer names to
	 */
	void GetSelectedLayerNames( OUT TArray< FName >& OutSelectedLayerNames ) const;

	/** 
	 *	Sets the specified array of ULayer objects as the currently selected layers
	 *
	 *	@param	InSelectedLayers	The layers to select
	 */
	void SetSelectedLayers( const TArray< TSharedPtr< FLayerViewModel > >& InSelectedLayers );

	/** Set the current selection to the specified Layer names */
	void SetSelectedLayers( const TArray< FName >& LayerNames );

	/** Set the current selection to the specified Layer */
	void SetSelectedLayer( const FName& LayerName );

	/** @return	The UICommandList supported by the LayersView */
	const TSharedRef< FUICommandList > GetCommandList() const;

	/**
	 *	Adds the specified actors to a new layer
	 *
	 *	@param	Actors	The actors to add to the new layer
	 */
	void AddActorsToNewLayer( TArray< TWeakObjectPtr< AActor > > Actors );

	/** Toggles the visibility of the given layer and those of the other layers selected with it */
	void ToggleLayerVisibility( const TSharedPtr< FLayerViewModel >& InLayer );

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/** Broadcasts whenever the number of layers changes */
	DECLARE_DERIVED_EVENT( FLayerCollectionViewModel, ULayersSubsystem::FOnLayersChanged, FOnLayersChanged );
	FOnLayersChanged& OnLayersChanged() { return LayersChanged; }

	/**	Broadcasts whenever the currently selected layers changes */
	DECLARE_EVENT( FLayerCollectionViewModel, FOnSelectionChanged );
	FOnSelectionChanged& OnSelectionChanged() { return SelectionChanged; }

	/**	Broadcasts whenever a rename is requested on the selected layers */
	DECLARE_EVENT( FLayerCollectionViewModel, FOnRenameRequested );
	FOnRenameRequested& OnRenameRequested() { return RenameRequested; }
private:

	/**  
	 *	FLayerCollectionViewModel Constructor
	 *
	 *	@param	InWorldLayers	The layer management logic object
	 *	@param	InEditor		The UEditorEngine to use
	 */
	FLayerCollectionViewModel( const TWeakObjectPtr< UEditorEngine >& InEditor );

	/** Initializes the LayersView for use */
	void Initialize();

	/**	Refreshes any cached information */
	void Refresh();

	/** Handles updating the viewmodel when one of its filters changes */
	void OnFilterChanged();

	/** Refreshes the Layers list */
	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty );

	/** Refreshes the Layers list */
	void OnResetLayers();

	/** Handles updating the internal viewmodels when new layers are created */
	void OnLayerAdded( const TWeakObjectPtr< ULayer >& AddedLayer );

	/** Handles updating the internal viewmodels when layers are deleted */
	void OnLayerDelete();

	/** Discards any viewmodels which are invalid */
	void DestructivelyPurgeInvalidViewModels( TArray< TWeakObjectPtr< ULayer > > &ActualLayers );

	/** Creates ViewModels for all the layers in the specified list */
	void CreateViewModels( const TArray< TWeakObjectPtr< ULayer > > &ActualLayers );

	/** Rebuilds the list of filtered layers */
	void RefreshFilteredLayers();

	/**	Sorts the filtered layers list */
	void SortFilteredLayers();

	/** Binds all layer browser commands to delegates */
	void BindCommands();

	/** Appends the selected layer names to the specified array */
	void AppendSelectLayerNames( TArray< FName >& OutLayerNames ) const;

	/** @return	A guaranteed unique name for a layer */
	FName GenerateUniqueLayerName() const;


private:

	/** Deletes the currently selected layers */
	void DeleteLayer_Executed();

	/** @return	whether the currently selected layers can be deleted */
	bool DeleteLayer_CanExecute() const;

	/** @return	creates a new empty layer */
	void CreateEmptyLayer_Executed();

	/** @return	whether a new layer empty layer can be created */
	bool CreateEmptyLayer_CanExecute() const;

	/** Adds the currently selected actors to the a new layer */
	void AddSelectedActorsToNewLayer_Executed();

	/** @return	whether the currently selected layers can be added to a new layer */
	bool AddSelectedActorsToNewLayer_CanExecute() const;

	/** Adds the currently selected actors to the selected layers */
	void AddSelectedActorsToSelectedLayer_Executed();

	/** @return	whether the currently selected actors can be added to the selected layers */
	bool AddSelectedActorsToSelectedLayer_CanExecute() const;

	/** Removes the selected actors from the selected layers */
	void RemoveSelectedActorsFromSelectedLayer_Executed();

	/** @return	whether the selected actors can be remove from the selected layers */
	bool RemoveSelectedActorsFromSelectedLayer_CanExecute() const;

	/** Selects the actors in the selected layers */
	void SelectActors_Executed();

	/** @return	whether the actors in the selected layers can be selected */
	bool SelectActors_CanExecute() const;

	/** Appends the actors of the selected layers to the editors current actor selection */
	void AppendActorsToSelection_Executed();

	/** @return	whether the actors of the selected layers can be added to the editors current actor selection */
	bool AppendActorsToSelection_CanExecute() const;

	/** Deselects the actors belonging to the selected layers */
	void DeselectActors_Executed();

	/** @return	whether the actors belonging to the selected layers can be deselected */
	bool DeselectActors_CanExecute() const;

	/** Toggles the selected layers visibility */
	void ToggleSelectedLayersVisibility_Executed();

	/** @return	whether the selected layers visibility can be toggled */
	bool ToggleSelectedLayersVisibility_CanExecute() const;

	/** Makes all layers visible */
	void MakeAllLayersVisible_Executed();

	/** @return	whether all layers can be made visible */
	bool MakeAllLayersVisible_CanExecute() const;

	/** Requests renaming of selected layers */
	void RequestRenameLayer_Executed();

	/** @return true if the layer can be renamed */
	bool RequestRenameLayer_CanExecute() const;

private:

	/** true if the LayersView is in the middle of refreshing */
	bool bIsRefreshing;

	/** The collection of filters used to restrict the layers shown in the LayersView */
	const TSharedRef< LayerFilterCollection > Filters;

	/** All layers shown in the LayersView */
	TArray< TSharedPtr< FLayerViewModel > > FilteredLayerViewModels;

	/** All layers managed by the LayersView */
	TArray< TSharedPtr< FLayerViewModel > > AllLayerViewModels;

	/** Currently selected layers */
	TArray< TSharedPtr< FLayerViewModel > > SelectedLayers;

	/** The list of commands with bound delegates for the layer browser */
	const TSharedRef< FUICommandList > CommandList;

	/** The UEditorEngine to use */
	const TWeakObjectPtr< UEditorEngine > Editor;

	/** The layer management logic object */
	ULayersSubsystem* WorldLayers;

	/**	Broadcasts whenever one or more layers changes */
	FOnLayersChanged LayersChanged;

	/**	Broadcasts whenever the currently selected layers changes */
	FOnSelectionChanged SelectionChanged;

	/**	Broadcasts whenever a rename is requested on the selected layers */
	FOnRenameRequested RenameRequested;
};

