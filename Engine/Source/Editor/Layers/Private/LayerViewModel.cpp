// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayerViewModel.h"
#include "Editor/EditorEngine.h"
#include "ScopedTransaction.h"
#include "Misc/DelegateFilter.h"

#define LOCTEXT_NAMESPACE "Layer"

DEFINE_LOG_CATEGORY_STATIC(LogLayerViewModel, Fatal, All);

FLayerViewModel::FLayerViewModel( const TWeakObjectPtr< ULayer >& InLayer, const TWeakObjectPtr< UEditorEngine >& InEditor )
	: Editor( InEditor )
	, WorldLayers(Editor.IsValid() ? Editor->GetEditorSubsystem<ULayersSubsystem>() : nullptr)
	, Layer( InLayer )
{
	// Sanity checks
	if (!Editor.IsValid())
	{
		UE_LOGF(LogLayerViewModel, Fatal, "This function requires Editor.IsValid() == true.");
	}
	else if (WorldLayers == nullptr)
	{
		UE_LOGF(LogLayerViewModel, Fatal, "This function requires Editor->GetEditorSubsystem<ULayersSubsystem>() to be already loaded rather than being a nullptr.");
	}
}


void FLayerViewModel::Initialize()
{
	WorldLayers->OnLayersChanged().AddSP( this, &FLayerViewModel::OnLayersChanged );

	if ( Editor.IsValid() )
	{
		Editor->RegisterForUndo(this);
	}

	RefreshActorStats();
}


FLayerViewModel::~FLayerViewModel()
{
	WorldLayers->OnLayersChanged().RemoveAll( this );

	if ( Editor.IsValid() )
	{
		Editor->UnregisterForUndo(this);
	}
}


FName FLayerViewModel::GetFName() const
{
	if( !Layer.IsValid() )
	{
		return NAME_None;
	}

	return Layer->GetLayerName();
}


FString FLayerViewModel::GetName() const
{
	FString name;
	if( Layer.IsValid() )
	{
		name = Layer->GetLayerName().ToString();
	}
	return name;
}

FText FLayerViewModel::GetNameAsText() const
{
	if( !Layer.IsValid() )
	{
		return FText::GetEmpty();
	}

	return FText::FromName(Layer->GetLayerName());
}


bool FLayerViewModel::IsVisible() const
{
	if( !Layer.IsValid() )
	{
		return false;
	}

	return Layer->IsVisible();
}


void FLayerViewModel::ToggleVisibility()
{
	if( !Layer.IsValid() )
	{
		return;
	}

	VisibilityToggledEvent.Broadcast( AsShared() );
}


bool FLayerViewModel::CanRenameTo( const FName& NewLayerName, FString& OutMessage ) const
{
	if (NewLayerName.IsNone())
	{
		OutMessage = LOCTEXT("EmptyLayerName", "Layer must be given a name").ToString();
		return false;
	}

	ULayer* FoundLayer;
	if ( WorldLayers->TryGetLayer( NewLayerName, FoundLayer ) && FoundLayer != Layer.Get() )
	{
		OutMessage = LOCTEXT("RenameFailed_AlreadyExists", "This layer already exists").ToString();
		return false;
	}

	return true;
}


void FLayerViewModel::RenameTo( const FName& NewLayerName )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	if( Layer->GetLayerName() == NewLayerName)
	{
		return;
	}

	int32 LayerIndex = 0;
	FName UniqueNewLayerName = NewLayerName;
	while( WorldLayers->IsLayer( UniqueNewLayerName ) )
	{
		UniqueNewLayerName = FName( *FString::Printf( TEXT( "%s_%d" ), *NewLayerName.ToString(), ++LayerIndex ) );
	}

	const FScopedTransaction Transaction( LOCTEXT("RenameTo", "Rename Layer") );

	WorldLayers->RenameLayer( Layer->GetLayerName(), UniqueNewLayerName );
}


bool FLayerViewModel::CanAssignActors( const TArray< TWeakObjectPtr<AActor> > Actors, FText& OutMessage ) const
{
	if( !Layer.IsValid() )
	{
		OutMessage = LOCTEXT("InvalidLayer", "Invalid Layer");
		return false;
	}

	FFormatNamedArguments LayerArgs;
	LayerArgs.Add(TEXT("LayerName"), FText::FromName(Layer->GetLayerName()));

	bool bHasValidActorToAssign = false;
	int32 bAlreadyAssignedActors = 0;
	for( auto ActorIt = Actors.CreateConstIterator(); ActorIt; ++ActorIt )
	{
		TWeakObjectPtr< AActor > Actor = *ActorIt;
		if( !Actor.IsValid() )
		{
			OutMessage = FText::Format(LOCTEXT("InvalidActors", "Cannot add invalid Actors to {LayerName}"), LayerArgs);
			return false;
		}

		FFormatNamedArguments ActorArgs;
		ActorArgs.Add(TEXT("ActorName"), FText::FromName(Actor->GetFName()));

		if( !WorldLayers->IsActorValidForLayer( Actor.Get() ) )
		{
			OutMessage = FText::Format(LOCTEXT("InvalidLayers", "Actor '{ActorName}' cannot be associated with Layers"), ActorArgs);
			return false;
		}

		if( Actor->Layers.Contains( Layer->GetLayerName() ) )
		{
			bAlreadyAssignedActors++;
		}
		else
		{
			bHasValidActorToAssign = true;
		}
	}

	if( bAlreadyAssignedActors == Actors.Num() )
	{
		OutMessage = FText::Format(LOCTEXT("AlreadyAssignedActors", "All Actors already assigned to {LayerName}"), LayerArgs);
		return false;
	}

	if( bHasValidActorToAssign )
	{
		OutMessage = FText::Format(LOCTEXT("AssignActors", "Assign Actors to {LayerName}"), LayerArgs);
		return true;
	}

	OutMessage = FText::GetEmpty();
	return false;
}


bool FLayerViewModel::CanAssignActor( const TWeakObjectPtr<AActor> Actor, FText& OutMessage ) const
{
	if( !Layer.IsValid() )
	{
		OutMessage = LOCTEXT("InvalidLayer", "Invalid Layer");
		return false;
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("LayerName"), FText::FromName(Layer->GetLayerName()));

	if( !Actor.IsValid() )
	{
		OutMessage = FText::Format(LOCTEXT("InvalidActor", "Cannot add invalid Actors to {LayerName}"), Args);
		return false;
	}

	Args.Add(TEXT("ActorName"), FText::FromName(Actor->GetFName()));

	if( !WorldLayers->IsActorValidForLayer( Actor.Get()) )
	{
		OutMessage = FText::Format(LOCTEXT("InvalidLayers", "Actor '{ActorName}' cannot be associated with Layers"), Args);
		return false;
	}

	if( Actor->Layers.Contains( Layer->GetLayerName() )  )
	{
		OutMessage = FText::Format(LOCTEXT("AlreadyAssignedActor", "Already assigned to {LayerName}"), Args);
		return false;
	}

	OutMessage = FText::Format(LOCTEXT("AssignActor", "Assign to {LayerName}"), Args);
	return true;
}


void FLayerViewModel::AppendActors( TArray< TWeakObjectPtr< AActor > >& InActors ) const
{
	if( !Layer.IsValid() )
	{
		return;
	}

	WorldLayers->AppendActorsFromLayer( Layer->GetLayerName(), InActors );
}


void FLayerViewModel::AppendActorsOfSpecificType( TArray< TWeakObjectPtr< AActor > >& InActors, const TWeakObjectPtr< UClass >& Class )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	struct Local
	{
		static bool ActorIsOfClass( const TWeakObjectPtr< AActor >& Actor, const TWeakObjectPtr< UClass > InClass )
		{
			return Actor->GetClass() == InClass.Get();
		}
	};

	TSharedRef< TDelegateFilter< const TWeakObjectPtr< AActor >& > > Filter = MakeShareable( new TDelegateFilter< const TWeakObjectPtr< AActor >& >( TDelegateFilter< const TWeakObjectPtr< AActor >& >::FPredicate::CreateStatic( &Local::ActorIsOfClass, Class ) ) );
	WorldLayers->AppendActorsFromLayer( Layer->GetLayerName(), InActors, Filter );
}


void FLayerViewModel::AddActor( const TWeakObjectPtr< AActor >& Actor )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("AddActor", "Add Actor to Layer") );
	WorldLayers->AddActorToLayer( Actor.Get(), Layer->GetLayerName() );
}


void FLayerViewModel::AddActors( const TArray< TWeakObjectPtr< AActor > >& Actors )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("AddActors", "Add Actors to Layer") );
	WorldLayers->AddActorsToLayer( Actors, Layer->GetLayerName() );
}


void FLayerViewModel::RemoveActors( const TArray< TWeakObjectPtr< AActor > >& Actors )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("RemoveActors", "Remove Actors from Layer") );
	WorldLayers->RemoveActorsFromLayer( Actors, Layer->GetLayerName() );
}


void FLayerViewModel::RemoveActor( const TWeakObjectPtr< AActor >& Actor )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("RemoveActor", "Remove Actor from Layer") );
	WorldLayers->RemoveActorFromLayer( Actor.Get(), Layer->GetLayerName() );
}


void FLayerViewModel::SelectActors( bool bSelect, bool bNotify, bool bSelectEvenIfHidden, const TSharedPtr< IFilter< const TWeakObjectPtr< AActor >& > >& Filter )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("SelectActors", "Select Actors in Layer") );

	const bool bNotifySelectNone = false;
	const bool bDeselectBSPSurfs = true;
	Editor->SelectNone( bNotifySelectNone, bDeselectBSPSurfs );

	WorldLayers->SelectActorsInLayer( Layer->GetLayerName(), bSelect, bNotify, bSelectEvenIfHidden, Filter );
}


FText FLayerViewModel::GetActorStatTotal( int32 StatsIndex ) const
{
	if( !Layer.IsValid() )
	{
		return FText::AsNumber(0);
	}

	if( ActorStats.Num() <= StatsIndex )
	{
		return LOCTEXT("InvalidActorStatTotal", "Invalid");
	}

	return FText::AsNumber(ActorStats[ StatsIndex ].Total);
}


void FLayerViewModel::SelectActorsOfSpecificType( const TWeakObjectPtr< UClass >& Class )
{
	if( !Layer.IsValid() )
	{
		return;
	}

	struct Local
	{
		static bool ActorIsOfClass( const TWeakObjectPtr< AActor >& Actor, const TWeakObjectPtr< UClass > InClass )
		{
			return Actor->GetClass() == InClass.Get();
		}
	};

	const bool bSelect = true;
	const bool bNotify = true; 
	const bool bSelectEvenIfHidden = true;
	TSharedRef< TDelegateFilter< const TWeakObjectPtr< AActor >& > > Filter = MakeShareable( new TDelegateFilter< const TWeakObjectPtr< AActor >& >( TDelegateFilter< const TWeakObjectPtr< AActor >& >::FPredicate::CreateStatic( &Local::ActorIsOfClass, Class ) ) );
	SelectActors( bSelect, bNotify, bSelectEvenIfHidden, Filter );
}


const TArray< FLayerActorStats >& FLayerViewModel::GetActorStats() const
{
	return ActorStats;
}


void FLayerViewModel::SetDataSource( const TWeakObjectPtr< ULayer >& InLayer )
{
	if( Layer == InLayer )
	{
		return;
	}

	Layer = InLayer;
	Refresh();
}


const TWeakObjectPtr< ULayer > FLayerViewModel::GetDataSource()
{
	return Layer;
}


void FLayerViewModel::OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
{
	if( Action != ELayersAction::Modify && Action != ELayersAction::Reset )
	{
		return;
	}

	if( ChangedLayer.IsValid() && ChangedLayer != Layer )
	{
		return;
	}

	if( Action == ELayersAction::Reset || ChangedProperty == TEXT( "ActorStats" ) )
	{
		RefreshActorStats();
	}

	ChangedEvent.Broadcast();
}


void FLayerViewModel::RefreshActorStats()
{
	ActorStats.Empty();

	if( !Layer.IsValid() )
	{
		return;
	}

	ActorStats.Append( Layer->GetActorStats() );

	struct FCompareLayerActorStats
	{
		FORCEINLINE bool operator()( const FLayerActorStats& Lhs, const FLayerActorStats& Rhs ) const 
		{ 
			int32 Result = Lhs.Type->GetFName().Compare( Rhs.Type->GetFName() );
			return Result > 0; 
		}
	};

	ActorStats.Sort( FCompareLayerActorStats() );
}


void FLayerViewModel::Refresh()
{
	OnLayersChanged( ELayersAction::Reset, NULL, NAME_None );
}


#undef LOCTEXT_NAMESPACE
