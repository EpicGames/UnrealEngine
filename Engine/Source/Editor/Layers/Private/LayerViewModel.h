// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "Layers/LayersSubsystem.h"
#include "Layers/Layer.h"

class AActor;
class UEditorEngine;

/**
 * The non-UI solution specific presentation logic for a single Layer
 */
class FLayerViewModel : public TSharedFromThis< FLayerViewModel >, public FEditorUndoClient
{

public:
	
	/** FLayerViewModel destructor */
	virtual ~FLayerViewModel();

	/**  
	 *	Factory method which creates a new FLayerViewModel object
	 *
	 *	@param	InLayer			The layer wrap
	 *	@param	InWorldLayers	The layer management logic object
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef< FLayerViewModel > Create( const TWeakObjectPtr< ULayer >& InLayer, const TWeakObjectPtr< UEditorEngine >& InEditor )
	{
		TSharedRef< FLayerViewModel > NewLayer( new FLayerViewModel( InLayer, InEditor ) );
		NewLayer->Initialize();

		return NewLayer;
	}


public:
	
	/**	@return	The Layer's display name as a FName */
	FName GetFName() const;

	/**	@return	The Layer's display name as a FString */
	FString GetName() const;

	/**	@return	The Layer's display name as a FString */
	FText GetNameAsText() const;

	/**	@return Whether the Layer is visible */
	bool IsVisible() const;

	/** 
	 *	Toggles the specified layer's visibility
	 */
	void ToggleVisibility();

	/**
	 *	Returns whether the layer can be assigned the specified name
	 *
	 *	@param	NewLayerName		The Layers new name
	 *	@param	OutMessage [OUT]	A returned description explaining the boolean result
	 *	@return						true then the name can be assigned
	 */
	bool CanRenameTo( const FName& NewLayerName, FString& OutMessage ) const;

	/**
	 *	Renames the Layer to the specified name
	 *
	 *	@param	NewLayerName	The Layers new name
	 */
	void RenameTo( const FName& NewLayerName );

	/**
	 *	Returns whether the specified actors can be assigned to the Layer
	 *
	 *	@param	Actors				The Actors to check if assignment is valid
	 *	@param	OutMessage [OUT]	A returned description explaining the boolean result
	 *	@return						if true then at least one actor can be assigned; 
	 *								If false, either Invalid actors were discovered or all actors are already assigned
	 */
	bool CanAssignActors( const TArray< TWeakObjectPtr<AActor> > Actors, FText& OutMessage ) const;

	/**
	 *	Returns whether the specified actor can be assigned to the Layer
	 *
	 *	@param	Actor				The Actor to check if assignment is valid
	 *	@param	OutMessage [OUT]	A returned description explaining the boolean result
	 *	@return						if true then at least one actor can be assigned; 
	 *								If false, either the Actor is Invalid or already assigned
	 */
	bool CanAssignActor( const TWeakObjectPtr<AActor> Actor, FText& OutMessage ) const;
	
	/**
	 *	Appends all of the actors associated with this layer to the specified list
	 * 
	 *	@param	InActors	the list to append to
	 */
	void AppendActors( TArray< TWeakObjectPtr< AActor > >& InActors ) const;

	/**
	 *	Appends all of the actors associated with this layer to the specified list
	 * 
	 *	@param	InActors	the list to append to
	 */
	void AppendActorsOfSpecificType( TArray< TWeakObjectPtr< AActor > >& InActors, const TWeakObjectPtr< UClass >& Class );

	/**
	 *	Adds the specified actor to the layer
	 *
	 *	@param	Actor	the actor to add
	 */
	void AddActor( const TWeakObjectPtr<AActor>& Actor );

	/** 
	 *	Adds the specified actors to the layer
	 *
	 *	@param	Actors		The actors to add
	 *	@param	LayerName	The layer to add the actors to
	 */
	void AddActors( const TArray< TWeakObjectPtr<AActor> >& Actors );

	/** 
	 *	Removes the specified actors from the layer
	 *
	 *	@param	Actors		The actors to add
	 *	@param	LayerName	The layer to add the actors to
	 */
	void RemoveActors( const TArray< TWeakObjectPtr<AActor> >& Actors );

	/**
	 *	Removes the specified actor from the layer
	 *
	 *	@param	Actor	the actor to remove
	 */
	void RemoveActor( const TWeakObjectPtr<AActor>& Actor );

	/**
	 *	Selects in the Editor all the Actors assigned to the Layer, based on the specified conditions.
	 *
	 *	@param	bSelect					if true actors will be selected; If false, actors will be deselected
	 *	@param	bNotify					if true the editor will be notified of the selection change; If false, the editor will not
	 *	@param	bSelectEvenIfHidden		if true actors that are hidden will be selected; If false, they will be skipped
	 *	@param	Filter					Only actors which pass the filters restrictions will be selected
	 */
	void SelectActors( bool bSelect, bool bNotify, bool bSelectEvenIfHidden, const TSharedPtr< IFilter< const TWeakObjectPtr< AActor >& > >& Filter );

	/** 
	 *	Retrieves the total number of actors of a specific type currently assigned to the Layer as a FString
	 *
	 *	@param	StatsIndex		The array index of the FLayerActorStats
	 *	@return					The total actors as a FString			
	 */
	FText GetActorStatTotal( int32 StatsIndex ) const;

	/**	
	 *	Selected the Actors assigned to the Layer that are of a certain type
	 */
	void SelectActorsOfSpecificType( const TWeakObjectPtr< UClass >& Class );

	/**	@return	An array of Actor stats regarding the Layer */
	const TArray< FLayerActorStats >& GetActorStats() const;
	
	/** Sets the ULayer this viewmodel should represent */
	void SetDataSource( const TWeakObjectPtr< ULayer >& InLayer );

	/** @return The ULayer this viewmodel represents */
	const TWeakObjectPtr< ULayer > GetDataSource();

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/** Broadcasts whenever the layer changes */
	DECLARE_EVENT( FLayerViewModel, FChangedEvent )
	FChangedEvent& OnChanged() { return ChangedEvent; }

	/** Broadcasts whenever renaming a layer is requested */
	DECLARE_EVENT( FLayerViewModel, FRenamedRequestEvent )
	FRenamedRequestEvent& OnRenamedRequest() { return RenamedRequestEvent; }
	void BroadcastRenameRequest() { RenamedRequestEvent.Broadcast(); }

	/** Broadcasts whenever a layer visibility toggle is requested */
	DECLARE_EVENT_OneParam( FLayerViewModel, FVisibilityToggledEvent, const TSharedPtr<FLayerViewModel>& )
	FVisibilityToggledEvent& OnVisibilityToggled() { return VisibilityToggledEvent; }

private:

	/**
	 *	FLayerViewModel Constructor
	 *
	 *	@param	InLayer			The Layer to represent
	 *	@param	InEditor		The UEditorEngine to use
	 */
	FLayerViewModel( const TWeakObjectPtr< ULayer >& InLayer, const TWeakObjectPtr< UEditorEngine >& InEditor );

	/** Initializes the FLayerViewModel for use */
	void Initialize();

	/** Called when a change occurs regarding Layers */
	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty );

	/** Refreshes any cached information in the FLayerViewModel */
	void Refresh();

	/** Refreshes the cached ActorStats */
	void RefreshActorStats();

	// Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override { Refresh(); }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:

	/** The Actor stats of the Layer */
	TArray< FLayerActorStats > ActorStats;

	/** The UEditorEngine to use */
	const TWeakObjectPtr< UEditorEngine > Editor;

	/** The layer management logic object */
	ULayersSubsystem* WorldLayers;

	/** The Layer this object represents */
	TWeakObjectPtr< ULayer > Layer;

	/** Broadcasts whenever the layer changes */
	FChangedEvent ChangedEvent;

	/** Broadcasts whenever a rename is requested */
	FRenamedRequestEvent RenamedRequestEvent;

	/** Broadcasts whenever a layer visibility toggle is requested */
	FVisibilityToggledEvent VisibilityToggledEvent;
};
