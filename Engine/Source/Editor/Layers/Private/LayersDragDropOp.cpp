// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayersDragDropOp.h"

#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"

struct FSlateBrush;


void FLayersDragDropOp::Construct()
{
	const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("Layer.Icon16x"));
	if (Layers.Num() == 1)
	{
		SetToolTip(FText::FromName(Layers[0]), Icon);
	}
	else
	{
		FText Text = FText::Format(NSLOCTEXT("FLayersDragDropOp", "MultipleFormat", "{0} Layers"), Layers.Num());
		SetToolTip(Text, Icon);
	}

	SetupDefaults();
	FDecoratedDragDropOp::Construct();
}
