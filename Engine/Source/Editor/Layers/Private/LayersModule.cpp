// Copyright Epic Games, Inc. All Rights Reserved.
#include "LayersModule.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "LayerCollectionViewCommands.h"
#include "ActorLayerCollectionViewModel.h"
#include "SActorLayerCloud.h"
#include "SLayerBrowser.h"


IMPLEMENT_MODULE( FLayersModule, Layers );

void FLayersModule::StartupModule()
{
	FLayersViewCommands::Register();
}


void FLayersModule::ShutdownModule()
{
	FLayersViewCommands::Unregister();
}


TSharedRef< SWidget > FLayersModule::CreateLayerBrowser()
{
	return SNew( SLayerBrowser );
}


TSharedRef< class SWidget> FLayersModule::CreateLayerCloud( const TArray< TWeakObjectPtr< AActor > >& Actors )
{
	TSharedRef< class FActorLayerCollectionViewModel > CloudViewModel( FActorLayerCollectionViewModel::Create( GEditor ) );
	CloudViewModel->SetActors( Actors );

	return SNew( SActorLayerCloud, CloudViewModel );
}

