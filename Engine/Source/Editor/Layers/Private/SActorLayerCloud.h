// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/AppStyle.h"
#include "ActorLayerCollectionViewModel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "ActorLayerViewModel.h"

#define LOCTEXT_NAMESPACE "LayerCloud"

class ULayer;

/**
 *	Displays a list of Layers in a "cloud" like format to the user
 */
class SActorLayerCloud : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SActorLayerCloud ) {}
	SLATE_END_ARGS()

	~SActorLayerCloud()
	{
		// Remove all delegates we registered
		ViewModel->OnLayersChanged().RemoveAll( this );
	}

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< FActorLayerCollectionViewModel >& InViewModel )
	{
		ViewModel = InViewModel;

		ChildSlot
		[
			SNew( SOverlay )
			+SOverlay::Slot()
			[
				SAssignNew( TagArea, SWrapBox )
				.UseAllottedSize( true )
			]

			+SOverlay::Slot()
			.VAlign( VAlign_Center )
			.HAlign( HAlign_Center )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NoLayersMessage", "No Layers") )
				.Visibility( this, &SActorLayerCloud::GetNoLayersMessageVisibility )
			]
		];

		Reconstruct();

		ViewModel->OnLayersChanged().AddSP( this, &SActorLayerCloud::OnLayersChanged );
	}


private:

	/**	Rebuilds the set of individual widgets representing each displayed Layer */
	void Reconstruct()
	{
		TagArea->ClearChildren();

		for( auto LayerIt = ViewModel->GetLayers().CreateConstIterator(); LayerIt; ++LayerIt )
		{
			const TSharedPtr< FActorLayerViewModel > Layer = *LayerIt;

			TagArea->AddSlot()
				.Padding( 4, 4 )
				[
					SNew( SBorder )
					.BorderBackgroundColor( FLinearColor(.2f,.2f,.2f,.2f) )
					.BorderImage( FAppStyle::GetBrush( TEXT( "LayerCloud.Item.BorderImage" ) ) )
					.Content()
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign( VAlign_Center )
							.Padding( 4, 0 )
							[
								SNew( STextBlock )
								.Font( FAppStyle::GetFontStyle( FName( "LayerCloud.Item.LabelFont" ) ) )
								.ColorAndOpacity( FSlateColor::UseForeground() )
								.Text( Layer.Get(), &FActorLayerViewModel::GetName )
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding( 2, 0 )
							[
								SNew( SButton )
								.ContentPadding( 0 )
								.VAlign( VAlign_Center )
								.OnClicked( this, &SActorLayerCloud::OnRemoveLayerFromActorsClicked, Layer )
								.ButtonStyle( FAppStyle::Get(), "NoBorder" )
								.ToolTipText( LOCTEXT("RemoveLayerButton", "Remove from Layer") )
								.Content()
								[
									SNew( SImage )
									.Image( FAppStyle::GetBrush( TEXT( "LayerCloud.Item.ClearButton" ) ) )
								]
							]
					]
				];
		}
	}

	/**
	 *	Called whenever a change is made to Layers
	 *
	 *	@param	Action				The action taken on one or more layers
	 *	@param	ChangedLayer		The layer that changed
	 *	@param	ChangedProperty		The property that changed
	 */
	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
	{
		if( Action == ELayersAction::Rename )
		{
			return;
		}

		Reconstruct();
	}

	/**
	 * Returns the current visibility for the NoLayers message
	 */
	EVisibility GetNoLayersMessageVisibility() const
	{
		return ( TagArea->GetChildren()->Num() == 0 ) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/**
	 * Removes all the selected actors from the currently exposed Layers
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	FReply OnRemoveAllLayersClicked()
	{
		ViewModel->RemoveActorsFromAllLayers();
		return FReply::Handled();
	}

	/**
	 * Removes all the selected actors from the clicked Layer
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	FReply OnRemoveLayerFromActorsClicked( const TSharedPtr< FActorLayerViewModel > Layer )
	{
		ViewModel->RemoveActorsFromLayer( Layer );
		return FReply::Handled();
	}


private:

	/**	The widget box that holds the layer tags */
	TSharedPtr< SWrapBox > TagArea;

	/**	The ViewModel for the SWidget */
	TSharedPtr< FActorLayerCollectionViewModel > ViewModel;
};

#undef LOCTEXT_NAMESPACE
