// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLayerBrowser.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Editor.h"
#include "SLayerStats.h"
#include "SceneOutlinerModule.h"

#include "Widgets/Input/SSearchBox.h"
#include "SceneOutlinerPublicTypes.h"

#define LOCTEXT_NAMESPACE "LayerBrowser"

void SLayerBrowser::Construct(const FArguments& InArgs)
{
	//@todo Create a proper ViewModel [7/27/2012 Justin.Sargent]
	Mode = ELayerBrowserMode::Layers;

	LayerCollectionViewModel = FLayerCollectionViewModel::Create(GEditor);
	SelectedLayersFilter = MakeShareable(new FActorsAssignedToSpecificLayersFilter());
	SelectedLayerViewModel = FLayerViewModel::Create(NULL, GEditor); //We'll set the datasource for this viewmodel later

	SearchBoxLayerFilter = MakeShareable(new LayerTextFilter(LayerTextFilter::FItemToStringArray::CreateSP(this, &SLayerBrowser::TransformLayerToString)));

	LayerCollectionViewModel->AddFilter(SearchBoxLayerFilter.ToSharedRef());
	LayerCollectionViewModel->OnLayersChanged().AddSP(this, &SLayerBrowser::OnLayersChanged);
	LayerCollectionViewModel->OnSelectionChanged().AddSP(this, &SLayerBrowser::UpdateSelectedLayer);
	LayerCollectionViewModel->OnRenameRequested().AddSP(this, &SLayerBrowser::OnRenameRequested);

	//////////////////////////////////////////////////////////////////////////
	//	Layers View Section
	SAssignNew(LayersSection, SBorder)
		.Padding(5)
		.BorderImage(FAppStyle::GetBrush("NoBrush"))
		.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.ToolTipText(LOCTEXT("FilterSearchToolTip", "Type here to search layers"))
				.HintText(LOCTEXT("FilterSearchHint", "Search Layers"))
				.OnTextChanged(this, &SLayerBrowser::OnFilterTextChanged)
			]

			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(LayersView, SLayersView, LayerCollectionViewModel.ToSharedRef())
					.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
					.ConstructContextMenu(FOnContextMenuOpening::CreateSP(this, &SLayerBrowser::ConstructLayerContextMenu))
					.HighlightText(SearchBoxLayerFilter.Get(), &LayerTextFilter::GetRawFilterText)
				]
		];

	//////////////////////////////////////////////////////////////////////////
	//	Layer Contents Header
	SAssignNew(LayerContentsHeader, SBorder)
		.BorderImage(FAppStyle::GetBrush("LayerBrowser.LayerContentsQuickbarBackground"))
		.Visibility(TAttribute< EVisibility >(this, &SLayerBrowser::GetLayerContentsHeaderVisibility))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 0, 2, 0))
			[
				SAssignNew(ToggleModeButton, SButton)
				.ContentPadding(FMargin(2, 0, 2, 0))
				.ButtonStyle(FAppStyle::Get(), "LayerBrowserButton")
				.OnClicked(this, &SLayerBrowser::ToggleLayerContents)
				.ForegroundColor(FSlateColor::UseForeground())
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(0, 1, 3, 1)
					[
						SNew(SImage)
						.Image(this, &SLayerBrowser::GetToggleModeButtonImageBrush)
						.ColorAndOpacity(this, &SLayerBrowser::GetInvertedForegroundIfHovered)
					]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ContentsLabel", "See Contents"))
							.Visibility(this, &SLayerBrowser::IsVisibleIfModeIs, ELayerBrowserMode::Layers)
							.ColorAndOpacity(this, &SLayerBrowser::GetInvertedForegroundIfHovered)
						]
				]
			]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SLayerBrowser::GetLayerContentsHeaderText)
					.Visibility(this, &SLayerBrowser::IsVisibleIfModeIs, ELayerBrowserMode::LayerContents)
				]

			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.FillWidth(1.0f)
				[
					SNew(SLayerStats, SelectedLayerViewModel.ToSharedRef())
				]
		];


	//////////////////////////////////////////////////////////////////////////
	//	Layer Contents Section
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked< FSceneOutlinerModule >("SceneOutliner");
	
	FSceneOutlinerInitializationOptions InitOptions;
	{
		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowParentTree = false;
		InitOptions.bShowCreateNewFolder = false;

		auto CustomDelete = [this](TArray<TWeakPtr<ISceneOutlinerTreeItem>> Items)
		{
			TArray<TWeakObjectPtr<AActor>> Actors;
			Actors.Reserve(Items.Num());
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : Items)
			{
				if (FActorTreeItem* ActorItem = Item.Pin()->CastTo<FActorTreeItem>())
				{
					Actors.Add(ActorItem->Actor);
				}
			}
			RemoveActorsFromSelectedLayer(Actors);
		};

		InitOptions.CustomDelete = FCustomSceneOutlinerDeleteDelegate::CreateLambda(CustomDelete);

		// Outliner Gutter
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Gutter(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0) );
		// Actor Label
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10) );
		// Layer Contents
		InitOptions.ColumnMap.Add(FSceneOutlinerLayerContentsColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 20,
			FCreateSceneOutlinerColumn::CreateSP( this, &SLayerBrowser::CreateCustomLayerColumn )) );

		InitOptions.Filters->Add(SelectedLayersFilter);
	}

	SAssignNew(LayerContentsSection, SBorder)
		.Padding(5)
		.BorderImage(FAppStyle::GetBrush("NoBrush"))
		.Content()
		[
			SceneOutlinerModule.CreateActorBrowser(InitOptions)
		];


	//////////////////////////////////////////////////////////////////////////
	//	Layer Browser
	ChildSlot
		[
			SAssignNew(ContentAreaBox, SVerticalBox)
			.IsEnabled_Lambda([]() { return !UWorld::IsPartitionedWorld(GWorld); })
		];

	SetupLayersMode();
}

FText SLayerBrowser::GetLayerContentsHeaderText() const
{
	return FText::Format(LOCTEXT("SelectedContentsLabel", "{0} Contents"), FText::FromString(SelectedLayerViewModel->GetName()));
}

void SLayerBrowser::OnFilterTextChanged( const FText& InNewText )
{
	SearchBoxLayerFilter->SetRawFilterText(InNewText);
	SearchBoxPtr->SetError(SearchBoxLayerFilter->GetFilterErrorText());
}

#undef LOCTEXT_NAMESPACE
