// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Misc/TextFilter.h"
#include "ActorsAssignedToSpecificLayersFilter.h"
#include "ISceneOutlinerColumn.h"
#include "SceneOutlinerLayerContentsColumn.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "SLayersView.h"
#include "SLayersCommandsMenu.h"
#include "EditorActorFolders.h"

class ISceneOutliner;

typedef TTextFilter< const TSharedPtr< FLayerViewModel >& > LayerTextFilter;

namespace ELayerBrowserMode
{
	enum Type
	{
		Layers,
		LayerContents,

		Count
	};
}


/**
 *	
 */
class SLayerBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SLayerBrowser ) {}
	SLATE_END_ARGS()

	~SLayerBrowser()
	{
		LayerCollectionViewModel->OnLayersChanged().RemoveAll( this );
		LayerCollectionViewModel->OnSelectionChanged().RemoveAll( this );
		LayerCollectionViewModel->OnRenameRequested().RemoveAll( this );
		LayerCollectionViewModel->RemoveFilter( SearchBoxLayerFilter.ToSharedRef() );
	}

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs );


protected:

	/** Overridden from SWidget: Called when a key is pressed down */
	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape && Mode == ELayerBrowserMode::LayerContents )
		{
			SetupLayersMode();			
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr< FDecoratedDragDropOp > DragOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		if (DragOp.IsValid())
		{
			DragOp->ResetToDefaultToolTip();
		}
	}

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		if (!SelectedLayerViewModel->GetDataSource().IsValid())
		{
			return FReply::Unhandled();
		}

		TArray<TWeakObjectPtr<AActor>> Actors;
		TSharedPtr<FActorDragDropOp> ActorDragOp = nullptr;
		TSharedPtr<FFolderDragDropOp> FolderDragOp = nullptr;

		if (const TSharedPtr<FCompositeDragDropOp> CompositeDragOp = DragDropEvent.GetOperationAs<FCompositeDragDropOp>())
		{
			ActorDragOp = CompositeDragOp->GetSubOp<FActorDragDropOp>();
			FolderDragOp = CompositeDragOp->GetSubOp<FFolderDragDropOp>();
		}
		else
		{
			ActorDragOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
			FolderDragOp = DragDropEvent.GetOperationAs<FFolderDragDropOp>();
		}
		
		if (ActorDragOp.IsValid() || FolderDragOp.IsValid())
		{
			const FGeometry LayerContentsHeaderGeometry = SWidget::FindChildGeometry( MyGeometry, LayerContentsHeader.ToSharedRef() );
			bool bValidDrop = LayerContentsHeaderGeometry.IsUnderLocation( DragDropEvent.GetScreenSpacePosition() );

			if( !bValidDrop && Mode == ELayerBrowserMode::LayerContents )
			{
				const FGeometry LayerContentsSectionGeometry = SWidget::FindChildGeometry( MyGeometry, LayerContentsSection.ToSharedRef() );
				bValidDrop = LayerContentsSectionGeometry.IsUnderLocation( DragDropEvent.GetScreenSpacePosition() );
			}

			if( bValidDrop && ActorDragOp.IsValid() && ActorDragOp->Actors.Num() > 0)
			{
				Actors = ActorDragOp->Actors;
			}

			if (bValidDrop && FolderDragOp.IsValid())
			{
				if (UWorld* World = FolderDragOp->World.Get())
				{
					FActorFolders::GetWeakActorsFromFolders(*World, FolderDragOp->Folders, Actors, FolderDragOp->RootObject);
				}
			}
		}

		if (Actors.Num() > 0)
		{
			bool bCanAssign = false;
			FText Message;
			if (Actors.Num() > 1)
			{
				bCanAssign = SelectedLayerViewModel->CanAssignActors(Actors, OUT Message);
			}
			else
			{
				bCanAssign = SelectedLayerViewModel->CanAssignActor(Actors[0], OUT Message);
			}

			if (bCanAssign)
			{
				if (ActorDragOp.IsValid())
				{
					ActorDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
				}
				if (FolderDragOp.IsValid())
				{
					FolderDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
				}
			}
			else
			{
				if (ActorDragOp.IsValid())
				{
					ActorDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
				}
				if (FolderDragOp.IsValid())
				{
					FolderDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
				}
			}
		}

		// We leave the event unhandled so the children of the ListView get a chance to grab the drag/drop
		return FReply::Unhandled();
	}

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		if (!SelectedLayerViewModel->GetDataSource().IsValid())
		{
			return FReply::Unhandled();
		}

		bool bHandled = false;
		TArray<TWeakObjectPtr<AActor>> ActorsToDrop;

		TSharedPtr<FActorDragDropOp> ActorDragOp = nullptr;
		TSharedPtr<FFolderDragDropOp> FolderDragOp = nullptr;

		if (const TSharedPtr<FCompositeDragDropOp> CompositeDragOp = DragDropEvent.GetOperationAs<FCompositeDragDropOp>())
		{
			ActorDragOp = CompositeDragOp->GetSubOp<FActorDragDropOp>();
			FolderDragOp = CompositeDragOp->GetSubOp<FFolderDragDropOp>();
		}
		else
		{
			ActorDragOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
			FolderDragOp = DragDropEvent.GetOperationAs<FFolderDragDropOp>();
		}

		if (ActorDragOp.IsValid())
		{
			const FGeometry LayerContentsHeaderGeometry = SWidget::FindChildGeometry(MyGeometry, LayerContentsHeader.ToSharedRef());
			bool bValidDrop = LayerContentsHeaderGeometry.IsUnderLocation(DragDropEvent.GetScreenSpacePosition());

			if (!bValidDrop && Mode == ELayerBrowserMode::LayerContents)
			{
				const FGeometry LayerContentsSectionGeometry = SWidget::FindChildGeometry(MyGeometry, LayerContentsSection.ToSharedRef());
				bValidDrop = LayerContentsSectionGeometry.IsUnderLocation(DragDropEvent.GetScreenSpacePosition());
			}

			if (bValidDrop)
			{
				ActorsToDrop = ActorDragOp->Actors;
				bHandled = true;
			}
		}

		if (FolderDragOp.IsValid())
		{
			if (UWorld* World = FolderDragOp->World.Get())
			{
				FActorFolders::GetWeakActorsFromFolders(*World, FolderDragOp->Folders, ActorsToDrop, FolderDragOp->RootObject);
				bHandled = true;
			}
		}

		if (ActorsToDrop.Num() > 0)
		{
			SelectedLayerViewModel->AddActors(ActorsToDrop);
		}

		return bHandled ? FReply::Handled() : FReply::Unhandled();
	}


private:
	
	void RemoveActorsFromSelectedLayer( const TArray< TWeakObjectPtr< AActor > >& Actors )
	{
		SelectedLayerViewModel->RemoveActors( Actors );
	}

	TSharedRef< ISceneOutlinerColumn > CreateCustomLayerColumn( ISceneOutliner& SceneOutliner ) const
	{
		return MakeShareable( new FSceneOutlinerLayerContentsColumn( SelectedLayerViewModel.ToSharedRef() ) );
	}

	FSlateColor GetInvertedForegroundIfHovered() const
	{
		static const FName InvertedForegroundName("InvertedForeground");
		return ( ToggleModeButton.IsValid() && ( ToggleModeButton->IsHovered() || ToggleModeButton->IsPressed() ) ) ? FAppStyle::GetSlateColor(InvertedForegroundName): FSlateColor::UseForeground();
	}

	const FSlateBrush* GetToggleModeButtonImageBrush() const
	{
		static const FName ExploreLayerContents("LayerBrowser.ExploreLayerContents");
		static const FName ReturnToLayersList("LayerBrowser.ReturnToLayersList");
		return ( Mode == ELayerBrowserMode::Layers ) ? FAppStyle::GetBrush( ExploreLayerContents ) : FAppStyle::GetBrush( ReturnToLayersList );
	}

	FText GetLayerContentsHeaderText() const;

	/**	Returns the visibility of the See Contents label */
	EVisibility IsVisibleIfModeIs( ELayerBrowserMode::Type DesiredMode ) const
	{
		return ( Mode == DesiredMode ) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetLayerContentsHeaderVisibility() const
	{
		return ( SelectedLayerViewModel->GetDataSource().IsValid() ) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FReply ToggleLayerContents()
	{
		switch( Mode )
		{
		default:
		case ELayerBrowserMode::Layers:
			SetupLayerContentsMode();
			break;

		case ELayerBrowserMode::LayerContents:
			SetupLayersMode();
			break;
		}

		return FReply::Handled();
	}

	void SetupLayersMode() 
	{
		ContentAreaBox->ClearChildren();
		ContentAreaBox->AddSlot()
		.FillHeight( 1.0f )
		[
			LayersSection.ToSharedRef()
		];

		ContentAreaBox->AddSlot()
		.AutoHeight()
		.VAlign( VAlign_Bottom )
		.MaxHeight( 23 )
		[
			LayerContentsHeader.ToSharedRef()
		];

		Mode = ELayerBrowserMode::Layers;
	}

	void SetupLayerContentsMode() 
	{
		ContentAreaBox->ClearChildren();
		ContentAreaBox->AddSlot()
		.AutoHeight()
		.VAlign( VAlign_Top )
		.MaxHeight( 23 )
		[
			LayerContentsHeader.ToSharedRef()
		];

		ContentAreaBox->AddSlot()
		.AutoHeight()
		.FillHeight( 1.0f )
		[
			LayerContentsSection.ToSharedRef()
		];

		Mode = ELayerBrowserMode::LayerContents;
	}

	void TransformLayerToString( const TSharedPtr< FLayerViewModel >& Layer, OUT TArray< FString >& OutSearchStrings ) const
	{
		if( !Layer.IsValid() )
		{
			return;
		}

		OutSearchStrings.Add( Layer->GetName() );
	}

	void UpdateLayerContentsFilter()
	{
		TArray< FName > LayerNames;
		LayerCollectionViewModel->GetSelectedLayerNames( LayerNames );
		SelectedLayersFilter->SetLayers( LayerNames );
	}

	void UpdateSelectedLayer()
	{
		UpdateLayerContentsFilter();

		if( LayerCollectionViewModel->GetSelectedLayers().Num() == 1 )
		{
			const auto Layers = LayerCollectionViewModel->GetSelectedLayers();
			check( Layers.Num() == 1 );
			SelectedLayerViewModel->SetDataSource( Layers[ 0 ]->GetDataSource() );
		}
		else
		{
			SelectedLayerViewModel->SetDataSource( NULL );
		}
	}

	void OnLayersChanged( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
	{
		if( Action != ELayersAction::Reset && Action != ELayersAction::Delete )
		{
			if( !ChangedLayer.IsValid() || SelectedLayerViewModel->GetDataSource() == ChangedLayer )
			{
				UpdateLayerContentsFilter();
			}

			return;
		}

		UpdateSelectedLayer();

		if( Mode == ELayerBrowserMode::LayerContents && !SelectedLayerViewModel->GetDataSource().IsValid() )
		{
			SetupLayersMode();
		}
	}
	
	/** Callback when layers want to be renamed */
	void OnRenameRequested()
	{
		LayersView->RequestRenameOnSelectedLayer();
	}

	TSharedPtr< SWidget > ConstructLayerContextMenu()
	{
		return SNew( SLayersCommandsMenu, LayerCollectionViewModel.ToSharedRef() );
	}

	void OnFilterTextChanged( const FText& InNewText );

private:

	/** */
	TSharedPtr< SButton > ToggleModeButton;

	/**	 */
	TSharedPtr< SVerticalBox > ContentAreaBox;

	/**	 */
	TSharedPtr< SBorder > LayersSection;

	/**	 */
	TSharedPtr< SBorder > LayerContentsSection;

	/**	 */
	TSharedPtr< SBorder > LayerContentsHeader;

	/** */
	TSharedPtr< class SSearchBox> SearchBoxPtr;

	/**	 */
	TSharedPtr< LayerTextFilter > SearchBoxLayerFilter;

	/**	 */
	TSharedPtr< FActorsAssignedToSpecificLayersFilter > SelectedLayersFilter;

	/**	 */
	TSharedPtr< FLayerCollectionViewModel > LayerCollectionViewModel;

	/**	 */
	ELayerBrowserMode::Type Mode;

	/**	 */
	TSharedPtr< FLayerViewModel > SelectedLayerViewModel;

	/** The layer view widget, displays all the layers in the level */
	TSharedPtr< SLayersView > LayersView;
};
