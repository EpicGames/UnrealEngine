// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Layers/Layer.h"
#include "LayerViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "LayerStats"

class AActor;

class SLayerStats : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS( SLayerStats ){}
	SLATE_END_ARGS()

	/** SLayerStats destructor */
	~SLayerStats()
	{
		// Remove all delegates we registered
		ViewModel->OnChanged().RemoveAll( this );
	}

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The layer the row widget is supposed to represent
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< FLayerViewModel >& InViewModel )
	{
		ViewModel = InViewModel;

		ChildSlot
		[
			SAssignNew( StatsArea, SHorizontalBox )
		];

		Reconstruct();

		ViewModel->OnChanged().AddSP( this, &SLayerStats::Reconstruct );
	}

	/**
	 *	Rebuilds the children widgets of the StatsArea
	 */
	void Reconstruct()
	{
		StatButtonWidgets.Empty();
		StatsArea->ClearChildren();

		const TArray< FLayerActorStats >& ActorStats = ViewModel->GetActorStats();
		for( int StatsIndex = 0; StatsIndex < ActorStats.Num(); ++StatsIndex )
		{
			const TWeakObjectPtr< UClass > StatsActorClass = ActorStats[ StatsIndex ].Type;

			TSharedPtr< SButton > LastCreatedButton;
			StatsArea->AddSlot()
			.AutoWidth()
			.Padding( 0.0f, 0.0f, 6.0f, 0.0f )
			[
				SAssignNew( LastCreatedButton, SButton )
				.ButtonStyle( FAppStyle::Get(), "ToggleButton" )
				.ContentPadding( FMargin( 1 ) )
				.ForegroundColor( FSlateColor::UseForeground() )
				.OnClicked( this, &SLayerStats::SelectLayerActorsOfSpecificType, StatsActorClass )
				.ToolTipText( this, &SLayerStats::GetStatButtonToolTipText, StatsActorClass )
				.Content()
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FSlateIconFinder::FindIconBrushForClass(ActorStats[StatsIndex].Type))
						.ColorAndOpacity( this, &SLayerStats::GetForegroundColorForButton, StatsIndex )
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 3.0f, 0.0f, 3.0f, 0.0f )
					[
						SNew(STextBlock)
						.Text( ViewModel.Get(), &FLayerViewModel::GetActorStatTotal, StatsIndex )
						.ColorAndOpacity( this, &SLayerStats::GetForegroundColorForButton, StatsIndex )
					]
				]
			];

			StatButtonWidgets.Add( LastCreatedButton );
		}
	}


private:

	/**
	 *	Gets the appropriate EVisibility for the specified button depending on the button's current state
	 *
	 *	@param	StatsIndex	The index into StatButtonWidgets to find the associated button
	 *	@return				The EVisibility
	 */
	EVisibility GetVisibility( int StatsIndex ) const
	{
		if( StatsIndex >= StatButtonWidgets.Num() )
		{
			return EVisibility::Collapsed;
		}

		const auto Button = StatButtonWidgets[ StatsIndex ];

		return ( Button.IsValid() && ( Button->IsHovered() || Button->IsPressed() ) ) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/**
	 *	Gets the appropriate SlateColor for the specified button depending on the button's current state
	 *
	 *	@param	StatsIndex	The index into StatButtonWidgets to find the associated button
	 *	@return				The ForegroundColor
	 */
	FSlateColor GetForegroundColorForButton( int StatsIndex ) const
	{
		if( StatsIndex >= StatButtonWidgets.Num() )
		{
			return FSlateColor::UseForeground();
		}

		const auto Button = StatButtonWidgets[ StatsIndex ];

		static const FName InvertedForegroundName("InvertedForeground");

		return ( Button.IsValid() && ( Button->IsHovered() || Button->IsPressed() ) ) ? FAppStyle::GetSlateColor(InvertedForegroundName): FSlateColor::UseForeground();
	}

	/**
	 *	Selected the Actors assigned to the Layer of the specified type
	 *
	 *	@param	Class	The Actor class to select by
	 */
	FReply SelectLayerActorsOfSpecificType( const TWeakObjectPtr< UClass > Class )
	{
		ViewModel->SelectActorsOfSpecificType( Class );
		return FReply::Handled();
	}

	/**
	 *	Remove the Actors assigned to the Layer of the specified type
	 *
	 *	@param	Class	The Actor class to select by
	 */
	FReply RemoveAllLayerActorsOfSpecificType( const TWeakObjectPtr< UClass > Class )
	{
		TArray< TWeakObjectPtr< AActor > > Actors;
		ViewModel->AppendActorsOfSpecificType( Actors, Class );
		ViewModel->RemoveActors( Actors );
		return FReply::Handled();
	}

	/**
	 *	Retrieves the appropriate tooltip text for the stats button of the specified index
	 *
	 *	@param Class	The Actor class which the stats are associated with
	 */
	FText GetStatButtonToolTipText( const TWeakObjectPtr< UClass > Class ) const
	{
		return FText::Format( LOCTEXT("StatButtonToolTipFmt", "Select All {0} Actors in {1}"), FText::FromName(Class->GetFName()), FText::FromString(ViewModel->GetName()) );
	}

	/**
	 *	Retrieves the appropriate tooltip text for the remove stats button of the specified index
	 *
	 *	@param Class	The Actor class which the stats are associated with
	 */
	FText GetRemoveStatButtonToolTipText( const TWeakObjectPtr< UClass > Class ) const
	{
		return LOCTEXT("RemoveStatButtonToolTip", "Remove All");
	}


private:

	/** The layer associated with this widget */
	TSharedPtr< FLayerViewModel > ViewModel;

	/** The Box widget holding the individual stats specific widgets */
	TSharedPtr< SHorizontalBox > StatsArea;

	/** The button widgets representing individual stats */
	TArray< TSharedPtr< SButton> > StatButtonWidgets;
};

#undef LOCTEXT_NAMESPACE
