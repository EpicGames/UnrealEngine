// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLayersCommandsMenu.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "LayerCollectionViewCommands.h"
#include "LayerCollectionViewModel.h"
#include "LayersModule.h"
#include "Layout/Children.h"
#include "Modules/ModuleManager.h"


#define LOCTEXT_NAMESPACE "LayersCommands"



void SLayersCommandsMenu::Construct(const FArguments& InArgs, const TSharedRef< FLayerCollectionViewModel > InViewModel/*, TSharedPtr< SLayersViewRow > InSingleSelectedRow */)
{
	ViewModel = InViewModel;
	const FLayersViewCommands& Commands = FLayersViewCommands::Get();

	// Get all menu extenders for this context menu from the layers module
	FLayersModule& LayersModule = FModuleManager::GetModuleChecked<FLayersModule>(TEXT("Layers"));
	TArray<FLayersModule::FLayersMenuExtender> MenuExtenderDelegates = LayersModule.GetAllLayersMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(ViewModel->GetCommandList()));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	// Build up the menu
	FMenuBuilder MenuBuilder(InArgs._CloseWindowAfterMenuSelection, ViewModel->GetCommandList(), MenuExtender);
	{
		MenuBuilder.BeginSection("LayersCreate", LOCTEXT("MenuHeader", "Layers"));
		{
			MenuBuilder.AddMenuEntry(Commands.CreateEmptyLayer);
			MenuBuilder.AddMenuEntry(Commands.AddSelectedActorsToNewLayer);
			MenuBuilder.AddMenuEntry(Commands.AddSelectedActorsToSelectedLayer);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("LayersRemoveActors");
		{
			MenuBuilder.AddMenuEntry(Commands.RemoveSelectedActorsFromSelectedLayer);
		}
		MenuBuilder.EndSection();

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, "DeleteLayer", LOCTEXT("DeleteLayer", "Delete Layer"), LOCTEXT("DeleteLayerToolTip", "Removes all actors from the selected layers and then deletes the layers"));
		MenuBuilder.AddMenuEntry(Commands.RequestRenameLayer);

		MenuBuilder.BeginSection("LayersSelection", LOCTEXT("SelectionMenuHeader", "Selection"));
		{
			MenuBuilder.AddMenuEntry(Commands.SelectActors);
			MenuBuilder.AddMenuEntry(Commands.AppendActorsToSelection);
			MenuBuilder.AddMenuEntry(Commands.DeselectActors);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("LayersVisibility", LOCTEXT("VisibilityMenuHeader", "Visibility"));
		{
			MenuBuilder.AddMenuEntry(Commands.MakeAllLayersVisible);
		}
		MenuBuilder.EndSection();
	}

	ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
}


#undef LOCTEXT_NAMESPACE
