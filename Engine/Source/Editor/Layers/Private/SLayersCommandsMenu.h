// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FLayerCollectionViewModel;

/**
 * The widget that represents a row in the LayersView's list view control.  Generates widgets for each column on demand.
 */
class SLayersCommandsMenu : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SLayersCommandsMenu )
		:_CloseWindowAfterMenuSelection( true ) {}

		SLATE_ARGUMENT( bool, CloseWindowAfterMenuSelection )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InImplementation	The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< FLayerCollectionViewModel > InViewModel/*, TSharedPtr< SLayersViewRow > InSingleSelectedRow */);

private:
	/** The UI logic of the LayersView that is not Slate specific */
	TSharedPtr< FLayerCollectionViewModel > ViewModel;
};

