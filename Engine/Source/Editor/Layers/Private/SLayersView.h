// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "LayerCollectionViewCommands.h"
#include "Widgets/Views/SHeaderRow.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "LayerCollectionViewModel.h"
#include "SLayersViewRow.h"
#include "LayersDragDropOp.h"
#include "EditorActorFolders.h"

#define LOCTEXT_NAMESPACE "LayersView"

typedef SListView< TSharedPtr< FLayerViewModel > > SLayersListView;

/**
 * A slate widget that can be used to display a list of Layers and perform various layers related actions
 */
class SLayersView : public SCompoundWidget
{
public:
	typedef SListView< TSharedPtr< FLayerViewModel > >::FOnGenerateRow FOnGenerateRow;

public:

	SLATE_BEGIN_ARGS( SLayersView ) {}
		SLATE_ATTRIBUTE( FText, HighlightText )
		SLATE_ARGUMENT( FOnContextMenuOpening, ConstructContextMenu )
		SLATE_EVENT( FOnGenerateRow, OnGenerateRow )
	SLATE_END_ARGS()

	SLayersView()
	: bUpdatingSelection( false )
	{

	}

	/** SLayersView destructor */
	~SLayersView()
	{
		// Remove all delegates we registered
		ViewModel->OnLayersChanged().RemoveAll( this );
		ViewModel->OnSelectionChanged().RemoveAll( this );
	}

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< FLayerCollectionViewModel >& InViewModel )
	{
		ViewModel = InViewModel;

		HighlightText = InArgs._HighlightText;
		FOnGenerateRow OnGenerateRowDelegate = InArgs._OnGenerateRow;

		if( !OnGenerateRowDelegate.IsBound() )
		{
			OnGenerateRowDelegate = FOnGenerateRow::CreateSP( this, &SLayersView::OnGenerateRowDefault );
		}

		TSharedRef< SHeaderRow > HeaderRowWidget =
			SNew( SHeaderRow )

			/** We don't want the normal header to be visible */
			.Visibility( EVisibility::Collapsed )

			/** Layer visibility column */
			+SHeaderRow::Column(LayersView::ColumnID_Visibility)
			.DefaultLabel(NSLOCTEXT("LayersView", "Visibility", "Visibility"))
			.FixedWidth(40.0f)

			/** LayerName label column */
			+SHeaderRow::Column(LayersView::ColumnID_LayerLabel)
			.DefaultLabel(LOCTEXT("Column_LayerNameLabel", "Layer"));

		ChildSlot
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.FillHeight( 1.0f )
				[
					SAssignNew( ListView, SLayersListView )

					// Enable multi-select if we're in browsing mode, single-select if we're in picking mode
					.SelectionMode( ESelectionMode::Multi )

					// Point the tree to our array of root-level items.  Whenever this changes, we'll call RequestTreeRefresh()
					.ListItemsSource( &ViewModel->GetLayers() )

					// Find out when the user selects something in the tree
					.OnSelectionChanged( this, &SLayersView::OnSelectionChanged )

					// Called when the user double-clicks with LMB on an item in the list
					.OnMouseButtonDoubleClick( this, &SLayersView::OnListViewMouseButtonDoubleClick )

					// Generates the actual widget for a tree item
					.OnGenerateRow( OnGenerateRowDelegate ) 

					// Use the level viewport context menu as the right click menu for list items
					.OnContextMenuOpening( InArgs._ConstructContextMenu )

					// Header for the tree
					.HeaderRow( HeaderRowWidget )

					// Items scrolled into view
					.OnItemScrolledIntoView( this, &SLayersView::OnItemScrolledIntoView)

					// Help text 
					.ToolTipText(LOCTEXT("HelpText", "Drag actors from the outliner or right click to add a new layer."))
				]
			];

		ViewModel->OnLayersChanged().AddSP( this, &SLayersView::RequestRefresh );
		ViewModel->OnSelectionChanged().AddSP( this, &SLayersView::UpdateSelection );
	}

	/** Requests a rename on the selected layer, first forcing the item to scroll into view */
	void RequestRenameOnSelectedLayer()
	{
		if(ListView->GetNumItemsSelected() == 1)
		{
			RequestedRenameLayer = ListView->GetSelectedItems()[0];
			ListView->RequestScrollIntoView(ListView->GetSelectedItems()[0]);
		}
	}

protected:

	/**
	 * Checks to see if this widget supports keyboard focus.  Override this in derived classes.
	 *
	 * @return  True if this widget can take keyboard focus
	 */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		return ViewModel->GetCommandList()->ProcessCommandBindings( InKeyEvent ) ? FReply::Handled() : FReply::Unhandled();
	}

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr< FDecoratedDragDropOp > DragOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		if (DragOp.IsValid())
		{
			DragOp->ResetToDefaultToolTip();
		}
	}

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr< FDecoratedDragDropOp > DecoratedDragDropOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		
		if (DecoratedDragDropOp.IsValid())
		{
			DecoratedDragDropOp->SetToolTip(LOCTEXT("OnDragOver", "Add Actors to New Layer"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
		}

		// We leave the event unhandled so the children of the ListView get a chance to grab the drag/drop
		return FReply::Unhandled();
	}

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		bool bHandled = false;
		TArray<TWeakObjectPtr<AActor>> ActorsToDrop;

		TSharedPtr<FActorDragDropOp> ActorDragOp = nullptr;
		TSharedPtr<FFolderDragDropOp> FolderDragOp = nullptr;

		if (const TSharedPtr<FCompositeDragDropOp> CompositeDragOp = DragDropEvent.GetOperationAs<FCompositeDragDropOp>())
		{
			ActorDragOp = CompositeDragOp->GetSubOp<FActorDragDropOp>();
			FolderDragOp = CompositeDragOp->GetSubOp<FFolderDragDropOp>();
		}
		else
		{
			ActorDragOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
			FolderDragOp = DragDropEvent.GetOperationAs<FFolderDragDropOp>();
		}

		if (ActorDragOp.IsValid())
		{
			ActorsToDrop = ActorDragOp->Actors;
			bHandled = true;
		}

		if (FolderDragOp.IsValid())
		{
			if (UWorld* World = FolderDragOp->World.Get())
			{
				FActorFolders::GetWeakActorsFromFolders(*World, FolderDragOp->Folders, ActorsToDrop, FolderDragOp->RootObject);

				bHandled = true;
			}
		}

		if (ActorsToDrop.Num() > 0)
		{
			ViewModel->AddActorsToNewLayer(ActorsToDrop);
		}

		return bHandled ? FReply::Handled() : FReply::Unhandled();
	}

	FReply OnDragRow(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && ViewModel->GetSelectedLayers().Num() > 0)
		{
			TSharedRef<FLayersDragDropOp> DragDropOp = MakeShared<FLayersDragDropOp>();

			for (TSharedPtr<FLayerViewModel> Layer : ViewModel->GetSelectedLayers())
			{
				FName LayerName = Layer->GetFName();
				if (LayerName != NAME_None)
				{
					DragDropOp->Layers.Add(LayerName);
				}
			}

			DragDropOp->Construct();
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}

		return FReply::Unhandled();
	}

private:

	/** 
	 *	Called by SListView to generate a table row for the specified item.
	 *
	 *	@param	Item		The FLayerViewModel to generate a TableRow widget for
	 *	@param	OwnerTable	The TableViewBase which will own the generated TableRow widget
	 *
	 *	@return The generated TableRow widget
	 */
	TSharedRef< ITableRow > OnGenerateRowDefault( const TSharedPtr< FLayerViewModel > Item, const TSharedRef< STableViewBase >& OwnerTable )
	{
		return SNew( SLayersViewRow, Item.ToSharedRef(), OwnerTable )
			.HighlightText( HighlightText )
			.OnDragDetected(this, &SLayersView::OnDragRow);
	}

	/**
	 *	Kicks off a Refresh of the LayersView
	 *
	 *	@param	Action				The action taken on one or more layers
	 *	@param	ChangedLayer		The layer that changed
	 *	@param	ChangedProperty		The property that changed
	 */
	void RequestRefresh( const ELayersAction::Type Action, const TWeakObjectPtr< ULayer >& ChangedLayer, const FName& ChangedProperty )
	{
		ListView->RequestListRefresh();
	}

	/**
	 *	Called whenever the viewmodels selection changes
	 */
	void UpdateSelection()
	{
		if( bUpdatingSelection )
		{
			return;
		}

		bUpdatingSelection = true;
		const auto& SelectedLayers = ViewModel->GetSelectedLayers();
		ListView->ClearSelection();
		for( auto LayerIter = SelectedLayers.CreateConstIterator(); LayerIter; ++LayerIter )
		{
			ListView->SetItemSelection( *LayerIter, true );
		}

		if( SelectedLayers.Num() == 1 )
		{
			ListView->RequestScrollIntoView( SelectedLayers[ 0 ] );
		}
		bUpdatingSelection = false;
	}

	/** 
	 *	Called by SListView when the selection has changed 
	 *
	 * @param	Item	The Layer affected by the selection change
	 * @param SelectInfo Provides context on how the selection changed
	 */
	void OnSelectionChanged( const TSharedPtr< FLayerViewModel > Item, ESelectInfo::Type SelectInfo )
	{
		if( bUpdatingSelection )
		{
			return;
		}

		bUpdatingSelection = true;
		ViewModel->SetSelectedLayers( ListView->GetSelectedItems() );
		bUpdatingSelection = false;
	}

	/** 
	 *	Called by SListView when the user double-clicks on an item 
	 *
	 *	@param	Item	The Layer that was double clicked
	 */
	void OnListViewMouseButtonDoubleClick( const TSharedPtr< FLayerViewModel > Item )
	{
		const FLayersViewCommands& Commands = FLayersViewCommands::Get();
		ViewModel->GetCommandList()->TryExecuteAction( Commands.SelectActors.ToSharedRef() );
	}

	/** Handler for when an item has scrolled into view after having been requested to do so */
	void OnItemScrolledIntoView(TSharedPtr<FLayerViewModel> LayerItem, const TSharedPtr<ITableRow>& Widget)
	{
		// Check to see if the layer wants to rename before requesting the rename.
		if(LayerItem == RequestedRenameLayer.Pin())
		{
			LayerItem->BroadcastRenameRequest();
			RequestedRenameLayer.Reset();
		}
	}


private:

	/**	Whether the view is currently updating the viewmodel selection */
	bool bUpdatingSelection;

	/** The UI logic of the LayersView that is not Slate specific */
	TSharedPtr< FLayerCollectionViewModel > ViewModel;

	/** Our list view used in the SLayersViews */
	TSharedPtr< SLayersListView > ListView;

	/** The string to highlight on any text contained in the view widget */
	TAttribute< FText > HighlightText;

	/** Used to defer a rename on a layer until after it has been scrolled into view */
	TWeakPtr< FLayerViewModel > RequestedRenameLayer;
};


#undef LOCTEXT_NAMESPACE
