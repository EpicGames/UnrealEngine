// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLayersViewRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "EditorActorFolders.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "LayersView"

void SLayersViewRow::Construct(const FArguments& InArgs, TSharedRef< FLayerViewModel > InViewModel, TSharedRef< STableViewBase > InOwnerTableView)
{
	ViewModel = InViewModel;

	HighlightText = InArgs._HighlightText;

	SMultiColumnTableRow< TSharedPtr< FLayerViewModel > >::Construct(FSuperRowType::FArguments().OnDragDetected(InArgs._OnDragDetected), InOwnerTableView);
}

SLayersViewRow::~SLayersViewRow()
{
	ViewModel->OnRenamedRequest().Remove(EnterEditingModeDelegateHandle);
}

TSharedRef< SWidget > SLayersViewRow::GenerateWidgetForColumn(const FName& ColumnID)
{
	TSharedPtr< SWidget > TableRowContent;

	if (ColumnID == LayersView::ColumnID_LayerLabel)
	{
		TableRowContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 1.0f, 3.0f, 1.0f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Layer.Icon16x")))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
				.Font(FAppStyle::GetFontStyle("LayersView.LayerNameFont"))
				.Text(ViewModel.Get(), &FLayerViewModel::GetNameAsText)
				.ColorAndOpacity(this, &SLayersViewRow::GetColorAndOpacity)
				.HighlightText(HighlightText)
				.ToolTipText(LOCTEXT("DoubleClickToolTip", "Double Click to Select All Actors"))
				.OnVerifyTextChanged(this, &SLayersViewRow::OnRenameLayerTextChanged)
				.OnTextCommitted(this, &SLayersViewRow::OnRenameLayerTextCommitted)
				.IsSelected(this, &SLayersViewRow::IsSelectedExclusively)
			]
		;

		EnterEditingModeDelegateHandle = ViewModel->OnRenamedRequest().AddSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}
	else if (ColumnID == LayersView::ColumnID_Visibility)
	{
		TableRowContent =
			SAssignNew(VisibilityButton, SButton)
			.ContentPadding(0)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &SLayersViewRow::OnToggleVisibility)
			.ToolTipText(LOCTEXT("VisibilityButtonToolTip", "Toggle Layer Visibility"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SLayersViewRow::GetVisibilityBrushForLayer)
			]
		;
	}
	else
	{
		checkf(false, TEXT("Unknown ColumnID provided to SLayersView"));
	}

	return TableRowContent.ToSharedRef();
}

void SLayersViewRow::OnRenameLayerTextCommitted(const FText& InText, ETextCommit::Type eInCommitType)
{
	if (!InText.IsEmpty())
	{
		ViewModel->RenameTo(*InText.ToString());
	}
}

bool SLayersViewRow::OnRenameLayerTextChanged(const FText& NewText, FText& OutErrorMessage)
{
	FString OutMessage;
	if (!ViewModel->CanRenameTo(*NewText.ToString(), OutMessage))
	{
		OutErrorMessage = FText::FromString(OutMessage);
		return false;
	}

	return true;
}

void SLayersViewRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr< FDecoratedDragDropOp > DragOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (DragOp.IsValid())
	{
		DragOp->ResetToDefaultToolTip();
	}
}

FReply SLayersViewRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TArray<TWeakObjectPtr<AActor>> Actors; 
	TSharedPtr<FActorDragDropOp> ActorDragOp = nullptr;
	TSharedPtr<FFolderDragDropOp> FolderDragOp = nullptr;

	if (const TSharedPtr<FCompositeDragDropOp> CompositeDragOp = DragDropEvent.GetOperationAs<FCompositeDragDropOp>())
	{
		ActorDragOp = CompositeDragOp->GetSubOp<FActorDragDropOp>();
		FolderDragOp = CompositeDragOp->GetSubOp<FFolderDragDropOp>();
	}
	else
	{
		ActorDragOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
		FolderDragOp = DragDropEvent.GetOperationAs<FFolderDragDropOp>();
	}

	if (ActorDragOp.IsValid() && ActorDragOp->Actors.Num() > 0)
	{
		Actors = ActorDragOp->Actors;
	}

	if (FolderDragOp.IsValid())
	{
		if (UWorld* World = FolderDragOp->World.Get())
		{
			FActorFolders::GetWeakActorsFromFolders(*World, FolderDragOp->Folders, Actors, FolderDragOp->RootObject);
		}
	}

	if (Actors.Num() > 0)
	{
		bool bCanAssign = false;
		FText Message;
		if (Actors.Num() > 1)
		{
			bCanAssign = ViewModel->CanAssignActors(Actors, OUT Message);
		}
		else
		{
			bCanAssign = ViewModel->CanAssignActor(Actors[0], OUT Message);
		}

		if (bCanAssign)
		{
			if (ActorDragOp.IsValid())
			{
				ActorDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
			}
			if (FolderDragOp.IsValid())
			{
				FolderDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
			}
		}
		else
		{
			if (ActorDragOp.IsValid())
			{
				ActorDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			}
			if (FolderDragOp.IsValid())
			{
				FolderDragOp->SetToolTip(Message, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SLayersViewRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bHandled = false;
	TArray<TWeakObjectPtr<AActor>> ActorsToDrop;

	if (const TSharedPtr<FActorDragDropOp> ActorDragOp = DragDropEvent.GetOperationAs<FActorDragDropOp>())
	{
		ActorsToDrop = ActorDragOp->Actors;
		bHandled = true;
	}
	else if (const TSharedPtr<FFolderDragDropOp> FolderDragOp = DragDropEvent.GetOperationAs<FFolderDragDropOp>())
	{
		if (UWorld* World = FolderDragOp->World.Get())
		{
			FActorFolders::GetWeakActorsFromFolders(*World, FolderDragOp->Folders, ActorsToDrop, FolderDragOp->RootObject);
			bHandled = true;
		}
	}
	else if (const TSharedPtr<FCompositeDragDropOp> CompositeDragOp = DragDropEvent.GetOperationAs<FCompositeDragDropOp>())
	{
		if (const TSharedPtr<FActorDragDropOp> ActorSubOp = CompositeDragOp->GetSubOp<FActorDragDropOp>())
		{
			ActorsToDrop = ActorSubOp->Actors;
			bHandled = true;
		}
		if (const TSharedPtr<FFolderDragDropOp> FolderSubOp = CompositeDragOp->GetSubOp<FFolderDragDropOp>())
		{
			if (UWorld* World = FolderDragOp->World.Get())
			{
				FActorFolders::GetWeakActorsFromFolders(*World, FolderSubOp->Folders, ActorsToDrop, FolderDragOp->RootObject);
				bHandled = true;
			}
		}
	}

	if (ActorsToDrop.Num() > 0)
	{
		ViewModel->AddActors(ActorsToDrop);
	}

	return bHandled ? FReply::Handled() : FReply::Unhandled();
}

FSlateColor SLayersViewRow::GetColorAndOpacity() const
{
	if (!FSlateApplication::Get().IsDragDropping())
	{
		return FSlateColor::UseForeground();
	}

	bool bCanAcceptDrop = false;
	TSharedPtr<FDragDropOperation> DragDropOp = FSlateApplication::Get().GetDragDroppingContent();

	if (DragDropOp.IsValid() && (DragDropOp->IsOfType<FActorDragDropOp>() || DragDropOp->IsOfType<FFolderDragDropOp>() || DragDropOp->IsOfType<FCompositeDragDropOp>()))
	{
		TArray<TWeakObjectPtr<AActor>> DraggedActors;

		TSharedPtr<FActorDragDropOp> ActorDragOp;
		TSharedPtr<FFolderDragDropOp> FolderDragOp;

		if (DragDropOp->IsOfType<FActorDragDropOp>())
		{
			ActorDragOp = StaticCastSharedPtr<FActorDragDropOp>(DragDropOp);
		}
		else if (DragDropOp->IsOfType<FFolderDragDropOp>())
		{
			FolderDragOp = StaticCastSharedPtr<FFolderDragDropOp>(DragDropOp);
		}
		else if (DragDropOp->IsOfType<FCompositeDragDropOp>())
		{
			if (const TSharedPtr<FCompositeDragDropOp> CompositeDrop = StaticCastSharedPtr<FCompositeDragDropOp>(DragDropOp))
			{
				ActorDragOp = CompositeDrop->GetSubOp<FActorDragDropOp>();
				FolderDragOp = CompositeDrop->GetSubOp<FFolderDragDropOp>();
			}
		}

		if (ActorDragOp.IsValid())
		{
			DraggedActors = ActorDragOp->Actors;
		}
		if (FolderDragOp.IsValid())
		{
			auto World = FolderDragOp->World;
			if (UWorld* WorldPtr = World.Get())
			{
				FActorFolders::GetWeakActorsFromFolders(*WorldPtr, FolderDragOp->Folders, DraggedActors, FolderDragOp->RootObject);
			}
		}

		if (DraggedActors.Num() > 0)
		{
			FText Message;
			bCanAcceptDrop = ViewModel->CanAssignActors(DraggedActors, OUT Message);
		}
	}

	return (bCanAcceptDrop) ? FSlateColor::UseForeground() : FLinearColor(0.30f, 0.30f, 0.30f);
}

const FSlateBrush* SLayersViewRow::GetVisibilityBrushForLayer() const
{
	if (ViewModel->IsVisible())
	{
		return IsHovered() ? FAppStyle::GetBrush("Level.VisibleHighlightIcon16x") :
			FAppStyle::GetBrush("Level.VisibleIcon16x");
	}
	else
	{
		return IsHovered() ? FAppStyle::GetBrush("Level.NotVisibleHighlightIcon16x") :
			FAppStyle::GetBrush("Level.NotVisibleIcon16x");
	}
}

#undef LOCTEXT_NAMESPACE
