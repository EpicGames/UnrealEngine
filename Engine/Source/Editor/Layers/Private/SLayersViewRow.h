// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "LayerViewModel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class SButton;

namespace LayersView
{
	/** IDs for list columns */
	static const FName ColumnID_LayerLabel( "Layer" );
	static const FName ColumnID_Visibility( "Visibility" );
}

/**
 * The widget that represents a row in the LayersView's list view control.  Generates widgets for each column on demand.
 */
class SLayersViewRow : public SMultiColumnTableRow< TSharedPtr< FLayerViewModel > >
{

public:

	SLATE_BEGIN_ARGS( SLayersViewRow ){}
		SLATE_ATTRIBUTE( FText, HighlightText )
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The layer the row widget is supposed to represent
	 * @param	InOwnerTableView	The owner of the row widget
	 */
	void Construct( const FArguments& InArgs,  TSharedRef< FLayerViewModel > InViewModel, TSharedRef< STableViewBase > InOwnerTableView );

	~SLayersViewRow();

protected:

	/**
	 * Constructs the widget that represents the specified ColumnID for this Row
	 * 
	 * @param ColumnName    A unique ID for a column in this TableView; see SHeaderRow::FColumn for more info.
	 *
	 * @return a widget to represent the contents of a cell in this row of a TableView. 
	 */
	virtual TSharedRef< SWidget > GenerateWidgetForColumn( const FName& ColumnID ) override;

	/** Callback when the SInlineEditableTextBlock is committed, to update the name of the layer this row represents. */
	void OnRenameLayerTextCommitted(const FText& InText, ETextCommit::Type eInCommitType);

	/** Callback when the SInlineEditableTextBlock is changed, to check for error conditions. */
	bool OnRenameLayerTextChanged(const FText& NewText, FText& OutErrorMessage);

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;


private:
	
	/**
	 *	Returns the Color and Opacity for displaying the bound Layer's Name.
	 *	The Color and Opacity changes depending on whether a drag/drop operation is occurring
	 *
	 *	@return	The SlateColor to render the Layer's name in
	 */
	FSlateColor GetColorAndOpacity() const;

	/**
	 *	Called when the user clicks on the visibility icon for a layer's row widget
	 *
	 *	@return	A reply that indicated whether this event was handled.
	 */
	FReply OnToggleVisibility()
	{
		ViewModel->ToggleVisibility();
		return FReply::Handled();
	}

	/**
	 *	Called to get the Slate Image Brush representing the visibility state of
	 *	the layer this row widget represents
	 *
	 *	@return	The SlateBrush representing the layer's visibility state
	 */
	const FSlateBrush* GetVisibilityBrushForLayer() const;

private:

	/** The layer associated with this row of data */
	TSharedPtr< FLayerViewModel > ViewModel;

	/**	The visibility button for the layer */
	TSharedPtr< SButton > VisibilityButton;

	/** The string to highlight on any text contained in the row widget */
	TAttribute< FText > HighlightText;

	/** Widget for displaying and editing the Layer name */
	TSharedPtr< SInlineEditableTextBlock > InlineTextBlock;

	/** Handle to the registered EnterEditingMode delegate */
	FDelegateHandle EnterEditingModeDelegateHandle;
};

