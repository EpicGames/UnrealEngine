// Copyright Epic Games, Inc. All Rights Reserved.


#include "SceneOutlinerLayerContentsColumn.h"

#include "ActorTreeItem.h"
#include "HAL/Platform.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Internationalization.h"
#include "LayerViewModel.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SNullWidget.h"

class AActor;
class SWidget;

#define LOCTEXT_NAMESPACE "SceneOutlinerLayerContentsColumn"


FSceneOutlinerLayerContentsColumn::FSceneOutlinerLayerContentsColumn( const TSharedRef< FLayerViewModel >& InViewModel )
	: ViewModel( InViewModel )
{

}

FName FSceneOutlinerLayerContentsColumn::GetID()
{
	return FName( "LayerContents" );
}

FName FSceneOutlinerLayerContentsColumn::GetColumnID()
{
	return GetID();
}


SHeaderRow::FColumn::FArguments FSceneOutlinerLayerContentsColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column( GetColumnID() )
		.FillWidth( 2.f )
		[
			SNew( SSpacer )
		];
}

const TSharedRef<SWidget> FSceneOutlinerLayerContentsColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FActorTreeItem* ActorItem = TreeItem->CastTo<FActorTreeItem>())
	{
		return SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "LayerBrowserButton")
			.ContentPadding(0)
			.OnClicked(this, &FSceneOutlinerLayerContentsColumn::OnRemoveFromLayerClicked, ActorItem->Actor)
			.ToolTipText(LOCTEXT("RemoveFromLayerButtonText", "Remove from Layer"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("LayerBrowser.Actor.RemoveFromLayer")))
			];
	}
	return SNullWidget::NullWidget;
}

FReply FSceneOutlinerLayerContentsColumn::OnRemoveFromLayerClicked( const TWeakObjectPtr< AActor > Actor )
{
	ViewModel->RemoveActor( Actor );
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
