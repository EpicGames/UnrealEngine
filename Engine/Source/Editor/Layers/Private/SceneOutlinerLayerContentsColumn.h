// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerColumn.h"
#include "Input/Reply.h"
#include "SceneOutlinerFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SHeaderRow.h"

class AActor;
class FLayerViewModel;
class SWidget;
template<typename ItemType> class STableRow;

/**
 * A custom column for the SceneOutliner which allows the user to remove actors from
 * the contents of a layer with a single click
 */
class FSceneOutlinerLayerContentsColumn : public ISceneOutlinerColumn
{

public:

	/**
	 *	Constructor
	 */
	FSceneOutlinerLayerContentsColumn( const TSharedRef< class FLayerViewModel >& InViewModel );

	virtual ~FSceneOutlinerLayerContentsColumn() = default;
	
	static FName GetID();

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation

	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;

	virtual const TSharedRef< SWidget > ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;

	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

private:
	FReply OnRemoveFromLayerClicked( const TWeakObjectPtr< AActor > Actor );
	
private:

	/**	 */
	const TSharedRef< FLayerViewModel > ViewModel;
};
