// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "UObject/NameTypes.h"

#define UE_API LAYERS_API

class FDragDropEvent;

/** Drag/drop operation for dragging layers in the editor */
class FLayersDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLayersDragDropOp, FDragDropOperation)

	/** The names of the layers being dragged */
	TArray<FName> Layers;

	UE_API virtual void Construct() override;
};

#undef UE_API
