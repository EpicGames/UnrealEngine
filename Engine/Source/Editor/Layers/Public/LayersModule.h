// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

class AActor;

/**
 * The module holding all of the UI related peices for Layers
 */
class FLayersModule : public IModuleInterface
{

public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates a Layer Browser widget
	 */
	virtual TSharedRef<class SWidget> CreateLayerBrowser();

	/**
	 *	Creates a widget that represents the layers the specified actors share in common as a cloud
	 */
	virtual TSharedRef< class SWidget> CreateLayerCloud( const TArray< TWeakObjectPtr< AActor > >& Actors );
	
	/** Delegates to be called to extend the layers menus */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<FExtender>, FLayersMenuExtender, const TSharedRef<FUICommandList>);
	virtual TArray<FLayersMenuExtender>& GetAllLayersMenuExtenders() {return LayersMenuExtenders;}

private:

	/** All extender delegates for the layers menus */
	TArray<FLayersMenuExtender> LayersMenuExtenders;
};


